// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CQTest : ModuleRules
{
	public CQTest(ReadOnlyTargetRules Target)
		: base(Target)
	{
		PublicDependencyModuleNames.Add("DeveloperSettings");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"NetCore",
					"Slate"
				 }
			);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EngineSettings",
					"LevelEditor",
					"UnrealEd"
			});
		}

		SetupIrisSupport(Target);
	}
}
