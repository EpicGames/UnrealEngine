// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTestSettings.h"
#include "CQTest.h"
#include "Tests/AutomationCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CQTestSettings)

const FString GenerateTestDirectory;
const FString DefaultTags;

namespace CQTestConsoleVariables 
{
static TAutoConsoleVariable<float> CVarCommandTimeout(
	CommandTimeoutName,
	CommandTimeout,
	TEXT("How long to wait on an asynchronous task before timing out in seconds"));

static TAutoConsoleVariable<float> CVarNetworkTimeout(
	NetworkTimeoutName,
	NetworkTimeout,
	TEXT("How long to wait on a network task before timing out in seconds"));

static TAutoConsoleVariable<float> CVarMapTestTimeout(
	MapTestTimeoutName,
	MapTestTimeout,
	TEXT("How long to wait on a map test before timing out in seconds"));
} // namespace CQTestConsoleVariables

TSharedPtr<FScopedTestEnvironment> UCQTestSettings::SetTestClassTimeouts(FTimespan Duration)
{
	TSharedPtr<FScopedTestEnvironment> TestEnvironment = FScopedTestEnvironment::Get();

	FString DurationString = FString::SanitizeFloat(Duration.GetSeconds());
	TestEnvironment->SetConsoleVariableValue(CQTestConsoleVariables::CommandTimeoutName, DurationString);
	TestEnvironment->SetConsoleVariableValue(CQTestConsoleVariables::NetworkTimeoutName, DurationString);
	TestEnvironment->SetConsoleVariableValue(CQTestConsoleVariables::MapTestTimeoutName, DurationString);

	return TestEnvironment;
}
