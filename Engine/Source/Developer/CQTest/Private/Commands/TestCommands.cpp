// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/TestCommands.h"
#include "CQTestSettings.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogCqTest, Log, All)

bool CQTEST_API FWaitUntil::Update()
{
	if (!bHasTimerStarted)
	{
		StartTime = FDateTime::UtcNow();
		bHasTimerStarted = true;
		if (Description != nullptr)
		{
			UE_LOGF(LogCqTest, Log, "Starting %ls", Description);
		}
	}
	if (TestRunner.HasAnyErrors())
	{
		return true;
	}

	auto Elapsed = FDateTime::UtcNow() - StartTime;
	if (Query())
	{
		if (Description)
		{
			UE_LOGF(LogCqTest, Log, "Finished %ls after %.0f milliseconds", Description, Elapsed.GetTotalMilliseconds());
		}
		return true;
	}
	else if (Elapsed >= Timeout)
	{
		if (Description)
		{
			TestRunner.AddError(*FString::Printf(TEXT("Timed out waiting for %s after %.0f milliseconds"), Description, Elapsed.GetTotalMilliseconds()));
		}
		else
		{
			TestRunner.AddError(*FString::Printf(TEXT("Latent command timed out after %.0f milliseconds"), Elapsed.GetTotalMilliseconds()));
		}
		return true;
	}
	return false;
}

FTimespan FWaitUntil::MakeTimeout(TOptional<FTimespan> InTimeout)
{
	if (InTimeout.IsSet())
	{
		return InTimeout.GetValue();
	}
	else if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CQTestConsoleVariables::CommandTimeoutName))
	{
		return FTimespan::FromSeconds(ConsoleVariable->GetFloat());
	}
	else
	{
		UE_LOGF(LogCqTest, Warning, "CVar '%ls' was not found. Defaulting to %f seconds.", CQTestConsoleVariables::CommandTimeoutName, CQTestConsoleVariables::CommandTimeout);
		return FTimespan::FromSeconds(CQTestConsoleVariables::CommandTimeout);
	}
}

bool CQTEST_API FWaitDelay::Update()
{
	if (!bHasTimerStarted)
	{
		EndTime = FDateTime::UtcNow() + Timeout;
		bHasTimerStarted = true;
		if (Description != nullptr)
		{
			UE_LOGF(LogCqTest, Log, "Starting %ls", Description);
		}
	}
	if (TestRunner.HasAnyErrors())
	{
		return true;
	}

	if (FDateTime::UtcNow() >= EndTime)
	{
		if (Description)
		{
			UE_LOGF(LogCqTest, Log, "Finished %ls", Description);
		}
		return true;
	}
	return false;
}

bool CQTEST_API FExecute::Update()
{
	bool bHasToRun = !TestRunner.HasAnyErrors() || FailureBehavior == ECQTestFailureBehavior::Run;
	if (Description)
	{
		UE_LOGF(LogCqTest, Log, "%ls %ls", bHasToRun? TEXT("Running") : TEXT("Skipping"), Description);
	}
	if (bHasToRun)
	{
		Func();
	}
	return true;
}

void CQTEST_API FRunSequence::Append(TSharedPtr<IAutomationLatentCommand> ToAdd)
{
	Commands.Add(ToAdd);
}

void CQTEST_API FRunSequence::AppendAll(TArray<TSharedPtr<IAutomationLatentCommand>> ToAdd)
{
	for (auto& Cmd : ToAdd)
	{
		Commands.Add(Cmd);
	}
}

void CQTEST_API FRunSequence::Prepend(TSharedPtr<IAutomationLatentCommand> ToAdd)
{
	Commands.Insert(ToAdd, 0);
}

bool CQTEST_API FRunSequence::Update()
{
	if (Commands.Num() == 0)
	{
		return true;
	}
	else
	{
		auto Command = Commands[0];
		Commands.RemoveAt(0); //Remove the command now, in case the command prepends other commands
		if (Command != nullptr && Command->Update() == false)
		{
			Commands.Insert(Command, 0);
			return false;
		}
		return Commands.IsEmpty();
	}
}
