// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ActorTestSpawner.h"
#include "TestGameInstance.h"

#include "EngineUtils.h"
#include "Engine/Engine.h"

namespace
{
	static constexpr bool bTriggerWorldDestroyCallbacks = true;

	void DestroyWorld(UWorld* GameWorld)
	{
		if (GameWorld->AreActorsInitialized())
		{
			for (AActor* const Actor : FActorRange(GameWorld))
			{
				if (Actor)
				{
					Actor->RouteEndPlay(EEndPlayReason::LevelTransition);
				}
			}
		}

		GEngine->ShutdownWorldNetDriver(GameWorld);
		GameWorld->DestroyWorld(bTriggerWorldDestroyCallbacks);
		GameWorld->SetPhysicsScene(nullptr);
		GEngine->DestroyWorldContext(GameWorld);
	}

	void DestroySpawnedActors(TArray<TWeakObjectPtr<AActor>>& SpawnedActors)
	{
		// Destroy actors in reverse order of creation
		for (int32 Index = SpawnedActors.Num() - 1; Index >= 0; Index--)
		{
			if (AActor* CastActor = SpawnedActors[Index].Get())
			{
				CastActor->Destroy(true);
			}
		}

		SpawnedActors.Empty();
	}

	void DestroySpawnedObjects(TArray<TWeakObjectPtr<UObject>>& InSpawnedObjects)
	{
		// Destroy objects in reverse order of creation
		for (int32 Index = InSpawnedObjects.Num() - 1; Index >= 0; Index--)
		{
			if (UObject* Obj = InSpawnedObjects[Index].Get())
			{
				Obj->ConditionalBeginDestroy();
			}
		}

		InSpawnedObjects.Empty();
	}
} // namespace

FActorTestSpawner::~FActorTestSpawner() {
	if (GameInstance)
	{
		GameInstance->Shutdown();
		GameInstance->RemoveFromRoot();
		GameInstance = nullptr;
	}

	DestroySpawnedActors(SpawnedActors);
	DestroySpawnedObjects(SpawnedObjects);

	if (GameWorld)
	{
		DestroyWorld(GameWorld);
		GameWorld->RemoveFromRoot();
		GameWorld = nullptr;
	}
}

UWorld* FActorTestSpawner::CreateWorld()
{
	FName WorldName = MakeUniqueObjectName(nullptr, UWorld::StaticClass(), NAME_None, EUniqueObjectNameOptions::GloballyUnique);
	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
	UWorld* Result = UWorld::CreateWorld(EWorldType::Game, false, WorldName, GetTransientPackage());
	check(Result != nullptr);
	Result->AddToRoot();
	WorldContext.SetCurrentWorld(Result);

	Result->InitializeActorsForPlay(FURL());
	check(Result->GetPhysicsScene() != nullptr);

	return Result;
}

void FActorTestSpawner::InitializeGameSubsystems()
{
	GameInstance = NewObject<UTestGameInstance>();
	GameInstance->AddToRoot();
	GameInstance->InitializeForTest(GetWorld());
}

UTestGameInstance* FActorTestSpawner::GetGameInstance()
{
	return GameInstance;
}
