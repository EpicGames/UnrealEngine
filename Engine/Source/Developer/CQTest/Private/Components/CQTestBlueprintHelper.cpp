// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CQTestBlueprintHelper.h"

#include "Misc/Paths.h"
#include "Engine/Blueprint.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UClass* FCQTestBlueprintHelper::GetBlueprintClass(const FString& Directory, const FString& Name)
{
	const FString FileName = FString::Printf(TEXT("%s.%s"), *Name, *Name);
	const FString Path = FPaths::Combine(*Directory, FileName);
	UObject* BpObject = Cast<UObject>(StaticLoadObject(UObject::StaticClass(), nullptr, *Path));
	check(BpObject);

	UBlueprint* AsBlueprint = Cast<UBlueprint>(BpObject);
	check(AsBlueprint);

	return AsBlueprint->GeneratedClass;
}

UObject* FCQTestBlueprintHelper::FindDataBlueprint(const FString& Directory, const FString& Name)
{
	static TMap<FString, FAssetData> BlueprintCache{};
	static TSet<FString> LoadedDirectories{};
	if (!LoadedDirectories.Contains(Directory))
	{
		TArray<FAssetData> AssetData;
		const FAssetRegistryModule& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		AssetRegistry.Get().GetAssetsByPath(FName(Directory), AssetData, true);
		for (const auto& Data : AssetData)
		{
			BlueprintCache.Add(Data.GetAsset()->GetFName().ToString(), Data);
		}

		LoadedDirectories.Add(Directory);
	}

	if (FAssetData* Data = BlueprintCache.Find(Name))
	{
		return Data->GetAsset();
	}

	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS