// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CQTestSlateComponent.h"

#include "Framework/Application/SlateApplication.h"
#include "Tests/AutomationCommon.h"

DEFINE_LOG_CATEGORY_STATIC(LogCQTestSlateComponent, Log, All);

FCQTestSlateComponent::FCQTestSlateComponent()
{
	checkf(FSlateApplication::IsInitialized(), TEXT("No Slate application initialized."));

	// Disable Slate from going into a sleep state forcing it to always tick
	TestEnvironment = FScopedTestEnvironment::Get();
	TestEnvironment->SetConsoleVariableValue(TEXT("Slate.AllowSlateToSleep"), TEXT("0"));

	TickDelegateHandle = FSlateApplication::Get().OnPostTick().AddRaw(this, &FCQTestSlateComponent::OnPostTick);
}

FCQTestSlateComponent::~FCQTestSlateComponent()
{
	if (FSlateApplication::IsInitialized() && TickDelegateHandle.IsValid())
	{
		FSlateApplication::Get().OnPostTick().Remove(TickDelegateHandle);
	}
}

bool FCQTestSlateComponent::HaveTicksElapsed(uint32 Ticks)
{
	// Early out checking that 0 ticks have elapsed as there is nothing to wait for
	if (Ticks == 0)
	{
		UE_LOGF(LogCQTestSlateComponent, Verbose, "Nothing to wait for as the expected elapsed ticks requested is 0.");
		return true;
	}

	if (!ExpectedTick.IsSet())
	{
		ExpectedTick = TickCounter + Ticks;

		UE_LOGF(LogCQTestSlateComponent, Verbose, "HaveTicksElapsed called for %d ticks.", Ticks);
		UE_LOGF(LogCQTestSlateComponent, Verbose, "Slate has ticked %d times.", TickCounter.load());
		UE_LOGF(LogCQTestSlateComponent, Verbose, "ExpectedTick set with a value of %d.", ExpectedTick.GetValue());
	}
	else if (ExpectedTick.GetValue() <= TickCounter)
	{
		UE_LOGF(LogCQTestSlateComponent, Verbose, "Slate ticked %d times and has met the expected tick goal of %d ticks.", TickCounter.load(), ExpectedTick.GetValue());

		ExpectedTick.Reset();
		return true;
	}

	return false;
}

void FCQTestSlateComponent::OnPostTick(const float InDeltaTime)
{
	TickCounter++;
}