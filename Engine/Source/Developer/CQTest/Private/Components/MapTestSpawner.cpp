// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MapTestSpawner.h"

#if WITH_AUTOMATION_TESTS
#include "Commands/TestCommands.h"
#include "CQTestSettings.h"
#include "Tests/AutomationCommon.h"
#include "GameDelegates.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogMapTest, Log, All);

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "HAL/FileManager.h"
#include "LevelEditor.h"
#include "LevelEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "Tests/AutomationEditorCommon.h"
#include "UnrealEdGlobals.h"

namespace {

const FString& GetTempMapDirectory()
{
	static const FString TempMapDirectory = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("CQTestMapTemp"));
	return TempMapDirectory;
}

/**
 * Generates a unique random 8 character map name.
 */
FString GenerateUniqueMapName()
{
	FString UniqueMapName = FGuid::NewGuid().ToString();
	UniqueMapName.LeftInline(8);

	return UniqueMapName;
}

/**
 * Cleans up all created resources.
 */
void CleanupTempResources()
{
	const bool bDirectoryMustExist = true;
	const bool bRemoveRecursively = true;
	const bool bWasDeleted = IFileManager::Get().DeleteDirectory(*GetTempMapDirectory(), bDirectoryMustExist, bRemoveRecursively);
	check(bWasDeleted);
}

} //anonymous
#endif // WITH_EDITOR

FMapTestSpawner::~FMapTestSpawner()
{
	// Only explicitly removing the 'EndPlayMapHandle' handle as either the `OnEndPlayMap` gets triggered and the Game/PIE Worlds are no longer valid
	// Or we are ending the test and the `FSpawnHelper` will handle cleaning up of the GameWorld
	if (EndPlayMapHandle.IsValid())
	{
		FGameDelegates::Get().GetEndPlayMapDelegate().Remove(EndPlayMapHandle);
		EndPlayMapHandle.Reset();
	}
}

TUniquePtr<FMapTestSpawner> FMapTestSpawner::CreateFromTempLevel(FTestCommandBuilder& InCommandBuilder)
{
#if WITH_EDITOR
	if (IsValid(GUnrealEd->PlayWorld))
	{
		UE_LOGF(LogMapTest, Verbose, "Active PIE session '%ls' needs to be shutdown before a creation of a new level can occur.", *GUnrealEd->PlayWorld->GetMapName());
		GUnrealEd->EndPlayMap();
	}

	FString MapName = GenerateUniqueMapName();
	FString MapPath = FPaths::Combine(GetTempMapDirectory(), MapName);
	FString NewLevelPackage = FPackageName::FilenameToLongPackageName(MapPath);

	ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();
	bool bWasTempLevelCreated = LevelEditorSubsystem->NewLevel(NewLevelPackage);
	check(bWasTempLevelCreated);

	TUniquePtr<FMapTestSpawner> Spawner = MakeUnique<FMapTestSpawner>(GetTempMapDirectory(), MapName);
	InCommandBuilder.OnTearDown([]() {
		// Create a new map to free up the reference to the map used during testing before cleaning up all temporary resources
		FAutomationEditorCommonUtils::CreateNewMap();
		CleanupTempResources();
	});
	return MoveTemp(Spawner);
#else
	checkf(false, TEXT("CreateFromTempLevel can't create a new level if WITH_EDITOR=false"));
	return nullptr;
#endif // WITH_EDITOR
}

void FMapTestSpawner::AddWaitUntilLoadedCommand(FAutomationTestBase* TestRunner, TOptional<FTimespan> Timeout)
{
	check(PieWorld == nullptr);

	FString PackagePath;
	const FString Path = FPaths::Combine(MapDirectory, MapName);
	bool bPackageExists = FPackageName::DoesPackageExist(Path, &PackagePath);
	checkf(bPackageExists, TEXT("Could not get package from path '%s'"), *Path);

	// We need to retrieve the LongPackageName from the PackagePath to be able to load the map for both Editor and Target builds
	FString LongPackageName, PackageConversionError;
	bool bFilenameConverted = FPackageName::TryConvertFilenameToLongPackageName(PackagePath, LongPackageName, &PackageConversionError);
	checkf(bFilenameConverted, TEXT("Could not get LongPackageName. Error: '%s'"), *PackageConversionError);

	bool bOpened = AutomationOpenMap(LongPackageName, true);
	check(bOpened);

	FTimespan TimeoutValue;
	if (Timeout.IsSet())
	{
		TimeoutValue = Timeout.GetValue();
	}
	else if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CQTestConsoleVariables::MapTestTimeoutName))
	{
		TimeoutValue = FTimespan::FromSeconds(ConsoleVariable->GetFloat());
	}
	else
	{
		UE_LOGF(LogMapTest, Warning, "CVar '%ls' was not found. Defaulting to %f seconds.", CQTestConsoleVariables::MapTestTimeoutName, CQTestConsoleVariables::MapTestTimeout);
		TimeoutValue = FTimespan::FromSeconds(CQTestConsoleVariables::MapTestTimeout);
	}

	ADD_LATENT_AUTOMATION_COMMAND(FWaitUntil(*TestRunner, [this]() -> bool {
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (!IsValid(World))
			{
				continue;
			}

			// We only want to set our PieWorld if the loaded World name matches our expected World name
			FString WorldMapName = FPackageName::GetShortName(World->GetMapName());
			WorldMapName = UWorld::RemovePIEPrefix(WorldMapName);
			if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && (WorldMapName.Equals(MapName)))
			{
				PieWorld = Context.World();
				EndPlayMapHandle = FGameDelegates::Get().GetEndPlayMapDelegate().AddRaw(this, &FMapTestSpawner::OnEndPlayMap);

				return true;
			}
		}

		return false;
	}, TimeoutValue));
}

UWorld* FMapTestSpawner::CreateWorld()
{
	checkf(PieWorld, TEXT("Must call AddWaitUntilLoadedCommand in BEFORE_TEST"));
	return PieWorld;
}

APawn* FMapTestSpawner::FindFirstPlayerPawn()
{
	APlayerController* PlayerController = GetWorld().GetFirstPlayerController();

	// There's a chance that we may not have a PlayerController spawned in the world
	if (!IsValid(PlayerController))
	{
		return nullptr;
	}

	return PlayerController->GetPawn();
}

void FMapTestSpawner::OnEndPlayMap()
{
	if (!IsValid(GEngine->GetCurrentPlayWorld()))
	{
		UE_LOGF(LogMapTest, Verbose, "Play session has ended.");
		GameWorld = nullptr;
		PieWorld = nullptr;

		FGameDelegates::Get().GetEndPlayMapDelegate().Remove(EndPlayMapHandle);
		EndPlayMapHandle.Reset();
	}
}

#endif // WITH_AUTOMATION_TESTS