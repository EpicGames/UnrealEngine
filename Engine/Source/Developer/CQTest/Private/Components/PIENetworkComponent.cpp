// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PIENetworkComponent.h"

#if ENABLE_PIE_NETWORK_TEST

#include "LevelEditor.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "UnrealEdGlobals.h"
#include "Tests/AutomationEditorCommon.h"

#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "GameFramework/GameMode.h"
#include "Modules/ModuleManager.h"

#include "GameMapsSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogNetworkTest, Log, All);

FBasePIENetworkComponent::FBasePIENetworkComponent(FAutomationTestBase* InTestRunner, FTestCommandBuilder& InCommandBuilder, bool IsInitializing, TOptional<FTimespan> Timeout)
	: TestRunner(InTestRunner)
	, CommandBuilder(&InCommandBuilder)
{
	if (IsInitializing)
	{
		return;
	}

	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder
		->Do(TEXT("Stop PIE"), [this]() { StopPie(); })
		.Then(TEXT("Create New Map"), [this]() { FAutomationEditorCommonUtils::CreateNewMap(); })
		.Then(TEXT("Start PIE"), [this]() { StartPie(); })
		.Until(TEXT("Set Worlds"), [this]() { return SetWorlds(); }, TimeoutValue)
		.Then(TEXT("Setup Packet Settings"), [this]() { SetPacketSettings(); })
		.Then(TEXT("Connect Clients to Server"), [this]() { ConnectClientsToServer(); })
		.Until(TEXT("Await Clients Ready"), [this]() { return AwaitClientsReady(); }, TimeoutValue)
		.OnTearDown(TEXT("Restore Editor State"), [this]() { RestoreState(); });
}

FBasePIENetworkComponent& FBasePIENetworkComponent::Then(TFunction<void()> Action)
{
	CommandBuilder->Then(Action);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::Do(TFunction<void()> Action)
{
	CommandBuilder->Do(Action);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::Then(const TCHAR* Description, TFunction<void()> Action)
{
	CommandBuilder->Then(Description, Action);
	return *this;
}
FBasePIENetworkComponent& FBasePIENetworkComponent::Do(const TCHAR* Description, TFunction<void()> Action)
{
	CommandBuilder->Do(Description, Action);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::Until(TFunction<bool()> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->Until(Query, TimeoutValue);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::Until(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->Until(Description, Query, TimeoutValue);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::StartWhen(TFunction<bool()> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->StartWhen(Query, TimeoutValue);
	return *this;
}

FBasePIENetworkComponent& FBasePIENetworkComponent::StartWhen(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->StartWhen(Description, Query, TimeoutValue);
	return *this;
}

void FBasePIENetworkComponent::StopPie()
{
	if (ServerState == nullptr)
	{
		TestRunner->AddError(TEXT("Failed to initialize Network Component"));
		return;
	}
	GUnrealEd->RequestEndPlayMap();
}

void FBasePIENetworkComponent::StartPie()
{
	ULevelEditorPlaySettings* PlaySettings = NewObject<ULevelEditorPlaySettings>();
	if (ServerState->bIsDedicatedServer)
	{
		PlaySettings->SetPlayNetMode(EPlayNetMode::PIE_Client);
		PlaySettings->SetPlayNumberOfClients(ServerState->ClientCount);
	}
	else
	{
		PlaySettings->SetPlayNetMode(EPlayNetMode::PIE_ListenServer);
		PlaySettings->SetPlayNumberOfClients(ServerState->ClientCount + 1); // The listen server counts as a client, so we need to add one more to get a real client as well
	}
	PlaySettings->bLaunchSeparateServer = ServerState->bIsDedicatedServer;
	PlaySettings->GameGetsMouseControl = false;
	PlaySettings->SetRunUnderOneProcess(true);

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	FRequestPlaySessionParams SessionParams;
	SessionParams.WorldType = EPlaySessionWorldType::PlayInEditor;
	SessionParams.DestinationSlateViewport = LevelEditorModule.GetFirstActiveViewport();
	SessionParams.EditorPlaySettings = PlaySettings;
	if (GameMode != nullptr)
	{
		SessionParams.GameModeOverride = GameMode;
	}
	else
	{
		SessionParams.GameModeOverride = AGameModeBase::StaticClass();
	}

	GUnrealEd->RequestPlaySession(SessionParams);
	GUnrealEd->StartQueuedPlaySessionRequest();
}

bool FBasePIENetworkComponent::SetWorlds() 
{
	auto IsValidContext = [](const FWorldContext& Context) -> bool {
		return Context.WorldType == EWorldType::PIE && 
			IsValid(Context.World()) && 
			IsValid(Context.World()->GetNetDriver());
	};

	auto IsValidServerWorld = [this](UWorld* World) -> bool {
		const bool bIsDedicated = ServerState->bIsDedicatedServer;
		const bool bExpectsDedicated = World->GetNetMode() == NM_DedicatedServer;
		return bExpectsDedicated == bIsDedicated;
	};

	auto IsClientWorldClaimed = [this](UWorld* World) -> bool {
		for (const auto& State : ClientStates) 
		{
			if (IsValid(State->World) && State->World == World) 
			{
				return true;
			}
		}
		return false;
	};

	int32 ClientWorldCount = 0;
	for (const auto& WorldContext : GEngine->GetWorldContexts())
	{
		if (!IsValidContext(WorldContext))
		{
			continue;
		}

		UWorld* World = WorldContext.World();
		if (World->GetNetDriver()->IsServer())
		{
			if (ServerState->World == nullptr)
			{
				if (!IsValidServerWorld(World)) 
				{
					TestRunner->AddError(TEXT("Failed to set up dedicated server. Does your game's editor module override the PIE settings?"));
					return true;
				}

				ServerState->World = World;
			}
		}
		else
		{
			if (!IsClientWorldClaimed(World))
			{
				bool bClaimed = false;
				for (const auto& State : ClientStates)
				{
					if (State->World == nullptr)
					{
						State->World = World;
						bClaimed = true;
						break;
					}
				}
				if (!bClaimed)
				{
					TestRunner->AddError(TEXT("Failed to claim client world. Network component was not able to be initialized."));
					return true;
				}

			}
			ClientWorldCount++;
		}
	}

	return IsValid(ServerState->World) && ClientWorldCount == ServerState->ClientCount;
}

void FBasePIENetworkComponent::SetPacketSettings() const
{
	if (PacketSimulationSettings)
	{
		ServerState->World->GetNetDriver()->SetPacketSimulationSettings(*PacketSimulationSettings);
		for (const auto& ClientState : ClientStates)
		{
			ClientState->World->GetNetDriver()->SetPacketSimulationSettings(*PacketSimulationSettings);
		}
	}
}

void FBasePIENetworkComponent::ConnectClientsToServer()
{
	auto& ServerConnections = ServerState->World->GetNetDriver()->ClientConnections;
	for(int32 ClientIndex = 0; ClientIndex < ServerState->ClientCount; ClientIndex++)
	{
		const int32 ClientLocalPort = ClientStates[ClientIndex]->World->GetNetDriver()->GetLocalAddr()->GetPort();
		TObjectPtr<UNetConnection>* ServerConnection = ServerConnections.FindByPredicate([ClientLocalPort](UNetConnection* ClientConnection) {
			return ClientConnection->GetRemoteAddr()->GetPort() == ClientLocalPort;
		});

		if (ServerConnection == nullptr)
		{
			TestRunner->AddError(TEXT("Failed to find connection to server for client. Network component was not able to be initialized."));
			return;
		}

		ServerState->ClientConnections[ClientIndex] = *ServerConnection;
	}
}

bool FBasePIENetworkComponent::AwaitClientsReady() const
{
	if (ServerState == nullptr || !IsValid(ServerState->World))
	{
		TestRunner->AddError(TEXT("Failed to get server state. Network component was not able to be initialized."));
		return true;
	}
	if (ServerState->World->GetNetDriver()->ClientConnections.Num() != ServerState->ClientCount)
	{
		return false;
	}
	for (const UNetConnection* ClientConnection : ServerState->World->GetNetDriver()->ClientConnections)
	{
		if (ClientConnection->ViewTarget == nullptr)
		{
			return false;
		}
	}

	return true;
}

void FBasePIENetworkComponent::RestoreState()
{
	if (ServerState != nullptr)
	{
		GUnrealEd->RequestEndPlayMap();
		StateRestorer.Restore();
	}
}

FTimespan FBasePIENetworkComponent::MakeTimeout(TOptional<FTimespan> Timeout)
{
	if (Timeout.IsSet())
	{
		return Timeout.GetValue();
	}
	else if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CQTestConsoleVariables::NetworkTimeoutName))
	{
		return FTimespan::FromSeconds(ConsoleVariable->GetFloat());
	}
	else
	{
		UE_LOGF(LogNetworkTest, Warning, "CVar '%ls' was not found. Defaulting to %f seconds.", CQTestConsoleVariables::NetworkTimeoutName, CQTestConsoleVariables::NetworkTimeout);
		return FTimespan::FromSeconds(CQTestConsoleVariables::NetworkTimeout);
	}
}

#endif // ENABLE_PIE_NETWORK_TEST