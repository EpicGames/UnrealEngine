// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PIENetworkTestStateRestorer.h"

#include "GameMapsSettings.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/WorldSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogPieNetworkStateRestorer, Log, All);

FPIENetworkTestStateRestorer::FPIENetworkTestStateRestorer(const FSoftClassPath InGameInstanceClass, const TSubclassOf<AGameModeBase> InGameMode)
{
	check(GWorld);
	if (InGameInstanceClass.IsValid())
	{
		UGameMapsSettings* GameMapSettings = GetMutableDefault<UGameMapsSettings>();
		if (GameMapSettings)
		{
			OriginalGameInstance = GameMapSettings->GameInstanceClass;
			GameMapSettings->GameInstanceClass = InGameInstanceClass;
		}
		else
		{
			UE_LOGF(LogPieNetworkStateRestorer, Error, "Unable to get UGameMapsSettings in when creating FPIENetworkTestStateRestorer");
		}
	}
	if (GWorld->GetWorldSettings() != nullptr)
	{
		OriginalGameMode = GWorld->GetWorldSettings()->DefaultGameMode;
		GWorld->GetWorldSettings()->DefaultGameMode = InGameMode;
	}
	if (!GWorld->HasAllFlags(RF_WasLoaded))
	{
		GWorld->SetFlags(RF_WasLoaded);
		SetWasLoadedFlag = true;
	}
}

void FPIENetworkTestStateRestorer::Restore() {
	if (GWorld && SetWasLoadedFlag)
	{
		GWorld->ClearFlags(RF_WasLoaded);
	}

	if (OriginalGameInstance.IsValid())
	{
		UGameMapsSettings* GameMapSettings = GetMutableDefault<UGameMapsSettings>();
		if (GameMapSettings)
		{
			GameMapSettings->GameInstanceClass = OriginalGameInstance;
		}
		else
		{
			UE_LOGF(LogPieNetworkStateRestorer, Error, "Unable to get UGameMapsSettings when destroying FPIENetworkTestStateRestorer");
		}
	}

	if (GWorld && OriginalGameMode != nullptr)
	{
		GWorld->GetWorldSettings()->DefaultGameMode = OriginalGameMode;
	}
}