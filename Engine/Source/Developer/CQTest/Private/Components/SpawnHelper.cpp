// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SpawnHelper.h"

FSpawnHelper::~FSpawnHelper() 
{
	if (GameWorld)
	{
		GameWorld->RemoveFromRoot();
		GameWorld = nullptr;
	}
}

UWorld& FSpawnHelper::GetWorld()
{
	if (!GameWorld)
	{
		GameWorld = CreateWorld();
		check(GameWorld);
		GameWorld->AddToRoot();
	}

	return *GameWorld;
}