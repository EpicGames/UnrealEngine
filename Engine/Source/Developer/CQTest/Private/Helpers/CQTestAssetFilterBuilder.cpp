// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CQTestAssetFilterBuilder.h"

namespace CQTestAssetHelper 
{

FAssetFilterBuilder& FAssetFilterBuilder::WithPackageName(const FName& PackageName)
{
	AssetRegistryFilter.PackageNames.Add(PackageName);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithPackagePath(const FName& PackagePath)
{
	FString SanitizedPackagePath = PackagePath.ToString();
	if (!SanitizedPackagePath.StartsWith("/"))
	{
		SanitizedPackagePath = FString::Format(TEXT("/{0}"), { SanitizedPackagePath });
	}
	if (SanitizedPackagePath.EndsWith("/"))
	{
		SanitizedPackagePath = SanitizedPackagePath.LeftChop(1);
	}

	AssetRegistryFilter.PackagePaths.Add(*SanitizedPackagePath);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithSoftObjectPath(const FSoftObjectPath& SoftObjectPath)
{
	AssetRegistryFilter.SoftObjectPaths.Add(SoftObjectPath);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithClassPath(const FTopLevelAssetPath& ClassPath)
{
	AssetRegistryFilter.ClassPaths.Add(ClassPath);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithTagAndValue(const FName& Tag, TOptional<FString> Value)
{
	AssetRegistryFilter.TagsAndValues.Add(Tag, Value);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithRecursiveClassPathsExclusion(const FTopLevelAssetPath& RecursiveClassPathsExclusion)
{
	AssetRegistryFilter.RecursiveClassPathsExclusionSet.Add(RecursiveClassPathsExclusion);
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::IncludeRecursivePaths()
{
	AssetRegistryFilter.bRecursivePaths = true;
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::IncludeRecursiveClasses()
{
	AssetRegistryFilter.bRecursiveClasses = true;
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::IncludeOnlyOnDiskAssets()
{
	AssetRegistryFilter.bIncludeOnlyOnDiskAssets = true;
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithPackageFlags(uint32 PackageFlags)
{
	AssetRegistryFilter.WithPackageFlags = PackageFlags;
	return *this;
}

FAssetFilterBuilder& FAssetFilterBuilder::WithoutPackageFlags(uint32 PackageFlags)
{
	AssetRegistryFilter.WithoutPackageFlags = PackageFlags;
	return *this;
}

FARFilter FAssetFilterBuilder::Build()
{
	return AssetRegistryFilter;
}

} // CQTestAssetHelper