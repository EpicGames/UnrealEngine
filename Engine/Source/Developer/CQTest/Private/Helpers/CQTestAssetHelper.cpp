// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/CQTestAssetHelper.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "Helpers/CQTestAssetFilterBuilder.h"

namespace 
{
DEFINE_LOG_CATEGORY_STATIC(LogCqTestAssets, Log, All);

bool FindAssets(const FARFilter& Filter, const FString& Name, TArray<FAssetData>& OutAssets)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.SearchAllAssets(true);
	}

	bool bWasSearchSuccess = false;
	TFunction<bool(const FAssetData& AssetData)> AssetSearch = [&OutAssets, &Name](const FAssetData& AssetData) {
		if ((Name.IsEmpty()) || (AssetData.AssetName.ToString() == Name) || (AssetData.GetObjectPathString() == Name))
		{
			OutAssets.Add(AssetData);
		}

		return true;
	};

	if (!Filter.IsEmpty())
	{
		bWasSearchSuccess = AssetRegistry.EnumerateAssets(Filter, AssetSearch);
	}
	else
	{
		bWasSearchSuccess = AssetRegistry.EnumerateAllAssets(AssetSearch);
	}

	if (!bWasSearchSuccess)
	{
		UE_LOGF(LogCqTestAssets, Error, "Error encountered while searching for asset.");
		return false;
	}

	return true;
}

TOptional<FAssetData> FindAsset(const FARFilter& Filter, const FString& Name) 
{
	TArray<FAssetData> FoundAssets;
	bool bSuccess = FindAssets(Filter, Name, FoundAssets);
	if (!bSuccess)
	{
		return NullOpt;
	}

	if (FoundAssets.IsEmpty())
	{
		UE_LOGF(LogCqTestAssets, Warning, "Asset name '%ls' not found.", *Name);
	}
	else if (FoundAssets.Num() > 1)
	{
		UE_LOGF(LogCqTestAssets, Warning, "Duplicate assets were found. May use the wrong one.");
	}

	if (FoundAssets.Num() > 0)
	{
		return FoundAssets[0];
	}

	return NullOpt;
}

} //anonymous

namespace CQTestAssetHelper 
{
	TOptional<FString> FindAssetPackagePathByName(const FString& AssetName)
	{
		return FindAssetPackagePathByName(FARFilter(), AssetName);
	}

	TOptional<FString> FindAssetPackagePathByName(const FARFilter& Filter, const FString& AssetName)
	{
		TOptional<FAssetData> Asset = FindAsset(Filter, AssetName);
		if (Asset.IsSet())
		{
			return Asset->PackagePath.ToString();
		}

		return NullOpt;
	}

	TArray<FAssetData> FindAssetsByFilter(const FARFilter& Filter)
	{
		TArray<FAssetData> FoundAssets;
		FindAssets(Filter, FString(), FoundAssets);
		return FoundAssets;
	}

	UClass* GetBlueprintClass(const FString& Name)
	{
		FARFilter Filter = FAssetFilterBuilder()
			.WithClassPath(UBlueprintCore::StaticClass()->GetClassPathName())
			.WithClassPath(UBlueprint::StaticClass()->GetClassPathName())
			.IncludeRecursiveClasses()
			.Build();

		return GetBlueprintClass(Filter, Name);
	}

	UClass* GetBlueprintClass(const FARFilter& Filter, const FString& Name)
	{
		TOptional<FAssetData> Asset = FindAsset(Filter, Name);
		if (Asset.IsSet())
		{
			if (UBlueprint* Bp = Cast<UBlueprint>(Asset->GetAsset()))
			{
				if (Bp->GeneratedClass != nullptr)
				{
					return Bp->GeneratedClass;
				}

				return Asset->GetClass();
			}
			UE_LOGF(LogCqTestAssets, Error, "Failed to load blueprint class for %ls", *Asset->AssetName.ToString());
		}

		return nullptr;
	}

	UObject* FindDataBlueprint(const FString& Name)
	{
		return FindDataBlueprint(FARFilter(), Name);
	}

	UObject* FindDataBlueprint(const FARFilter& Filter, const FString& Name)
	{
		TOptional<FAssetData> Asset = FindAsset(Filter, Name);
		if (Asset.IsSet())
		{
			return Asset->GetAsset();
		}
	
		return nullptr;
	}

} // CQTestAssetHelper