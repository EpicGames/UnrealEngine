// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/App.h"
#include "Misc/Paths.h"

namespace TestDirectoryGenerator
{

	void SplitPath(const FString& Path, FString* Lhs, FString* Rhs, FString* SourceType)
	{
		static const FString Plugins = TEXT("/Plugins/");
		static const FString Platforms = TEXT("/Platforms/");
		static const FString Source = TEXT("/Source/");
		static const FString Slash = TEXT("/");

		FString SplitString = Source;
		if (Path.Find(Plugins) != INDEX_NONE)
		{
			Path.Split(Plugins, Lhs, Rhs);
			*SourceType = TEXT("Plugins");
		}
		else if (Path.Find(Platforms) != INDEX_NONE)
		{
			Path.Split(Platforms, Lhs, Rhs);
			*SourceType = TEXT("Platforms");
		}
		else if (Path.Find(Source) != INDEX_NONE)
		{
			Path.Split(Source, Lhs, Rhs);
			*SourceType = TEXT("Source");
			SplitString = Slash;
		}
		else
		{
			checkf(false, TEXT("%s Should contain Plugins, Platforms, or Source"), *Path);
			*Lhs = TEXT("Unknown");
			*Rhs = TEXT("Unknown");
			*SourceType = TEXT("Unknown");
			return;
		}

		Lhs->Split(Slash, nullptr, Lhs, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		if (Lhs->IsEmpty())
		{
			*Lhs = TEXT("Unknown");
		}
		Rhs->Split(SplitString, Rhs, nullptr);
		if (Rhs->IsEmpty())
		{
			*Rhs = TEXT("Unknown");
		}
	}

	CQTEST_API FString Generate(const FString& Filename)
	{
		static const FString Engine = TEXT("Engine/");
		const FString Path(Filename.Replace(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive).Replace(TEXT("../"), *Engine, ESearchCase::CaseSensitive));

		FString Before, After, SourceType;
		SplitPath(Path, &Before, &After, &SourceType);
		After = After.Replace(TEXT("/"), TEXT("."));
		return FString::Printf(TEXT("%s.%s.%s"), *Before, *SourceType, *After);
	}
}