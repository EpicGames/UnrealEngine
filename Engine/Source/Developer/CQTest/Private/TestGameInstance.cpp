// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestGameInstance.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestGameInstance)

UTestGameInstance::UTestGameInstance(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
}

void UTestGameInstance::InitializeForTest(UWorld& InWorld) {
	FWorldContext* TestWorldContext = GEngine->GetWorldContextFromWorld(&InWorld);
	check(TestWorldContext);
	WorldContext = TestWorldContext;

	WorldContext->OwningGameInstance = this;
	InWorld.SetGameInstance(this);

	Init();
}
