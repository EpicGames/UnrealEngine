// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "UObject/NameTypes.h"

namespace CQTestCondition
{
	// Equal

	template <typename TValue, typename TOtherValue>
	inline bool IsEqual(const TValue& A, const TOtherValue& B)
	{
		return A == B;
	}

	inline bool IsEqual(const TCHAR* A, const TCHAR* B)
	{
		return FCString::Strcmp(A, B) == 0;
	}
	inline bool IsEqual(const TCHAR* A, const FString& B)
	{
		return IsEqual(A, *B);
	}
	inline bool IsEqual(const FString& A, const TCHAR* B)
	{
		return IsEqual(*A, B);
	}
	inline bool IsEqual(const FString& A, const FString& B)
	{
		return IsEqual(*A, *B);
	}

	inline bool IsEqualIgnoreCase(const TCHAR* A, const TCHAR* B)
	{
		return FCString::Stricmp(A, B) == 0;
	}
	inline bool IsEqualIgnoreCase(const TCHAR* A, const FString& B)
	{
		return IsEqualIgnoreCase(A, *B);
	}
	inline bool IsEqualIgnoreCase(const FString& A, const TCHAR* B)
	{
		return IsEqualIgnoreCase(*A, B);
	}
	inline bool IsEqualIgnoreCase(const FString& A, const FString& B)
	{
		return IsEqualIgnoreCase(*A, *B);
	}

	// Not Equal

	template <typename TValue, typename TOtherValue>
	inline bool IsNotEqual(const TValue& A, const TOtherValue& B)
	{
		return A != B;
	}

	inline bool IsNotEqual(const TCHAR* A, const TCHAR* B)
	{
		return FCString::Strcmp(A, B) != 0;
	}
	inline bool IsNotEqual(const TCHAR* A, const FString& B)
	{
		return IsNotEqual(A, *B);
	}
	inline bool IsNotEqual(const FString& A, const TCHAR* B)
	{
		return IsNotEqual(*A, B);
	}
	inline bool IsNotEqual(const FString& A, const FString& B)
	{
		return IsNotEqual(*A, *B);
	}

	inline bool IsNotEqualIgnoreCase(const TCHAR* A, const TCHAR* B)
	{
		return FCString::Stricmp(A, B) != 0;
	}
	inline bool IsNotEqualIgnoreCase(const TCHAR* A, const FString& B)
	{
		return IsNotEqualIgnoreCase(A, *B);
	}
	inline bool IsNotEqualIgnoreCase(const FString& A, const TCHAR* B)
	{
		return IsNotEqualIgnoreCase(*A, B);
	}
	inline bool IsNotEqualIgnoreCase(const FString& A, const FString& B)
	{
		return IsNotEqualIgnoreCase(*A, *B);
	}

	// Approximation

	inline bool IsNearlyEqual(const float A, const float B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return FMath::IsNearlyEqual(A, B, Tolerance);
	}
	inline bool IsNearlyEqual(const double A, const double B, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return FMath::IsNearlyEqual(A, B, Tolerance);
	}
	inline bool IsNearlyEqual(const FVector A, const FVector B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Equals(B, Tolerance);
	}
	inline bool IsNearlyEqual(const FTransform A, const FTransform B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Equals(B, Tolerance);
	}
	inline bool IsNearlyEqual(const FRotator A, const FRotator B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return A.Equals(B, Tolerance);
	}

	inline bool IsNotNearlyEqual(const float A, const float B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return !FMath::IsNearlyEqual(A, B, Tolerance);
	}
	inline bool IsNotNearlyEqual(const double A, const double B, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return !FMath::IsNearlyEqual(A, B, Tolerance);
	}
	inline bool IsNotNearlyEqual(const FVector A, const FVector B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return !A.Equals(B, Tolerance);
	}
	inline bool IsNotNearlyEqual(const FTransform A, const FTransform B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return !A.Equals(B, Tolerance);
	}
	inline bool IsNotNearlyEqual(const FRotator A, const FRotator B, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return !A.Equals(B, Tolerance);
	}
}
