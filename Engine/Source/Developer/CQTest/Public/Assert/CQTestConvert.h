// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"
#include <sstream>
#include <type_traits>

namespace CQTestConvert
{
	using PrintStream = TStringBuilder<64>;
}

namespace
{
	template <typename T, typename = void>
	struct THasToString : std::false_type
	{
	};

	template<typename T>
	struct THasToString<T, std::void_t<decltype(std::declval<T>().ToString())>>
		: std::is_same<FString, decltype(std::declval<T>().ToString())>::type
	{
	};

	template<typename T>
	class THasOStream
	{
		template <typename C, typename = decltype(operator<<(std::declval<CQTestConvert::PrintStream&>(), std::declval<C>()))>
		static std::true_type test(int);
		template<typename C>
		static std::false_type test(...);

	public:
		static constexpr bool value = decltype(test<T>(0))::value;
	};

	static_assert(std::is_same<FString, decltype(std::declval<FName>().ToString())>::value);
	static_assert(THasOStream<int>::value, "int should have an OStream operator");
	static_assert(THasOStream<int32>::value, "int32 should have an OStream operator");

	struct StructWithToString
	{
		FString ToString() {
			return FString();
		}
	};

	struct StructWithConstToString
	{
		FString ToString() const {
			return FString();
		}
	};

	static_assert(THasToString<StructWithToString>::value, "Struct with ToString should have ToString");
	static_assert(THasToString<StructWithConstToString>::value, "Struct with ToString const should have ToString");

	struct StructWithToStringWrongReturnType
	{
		int ToString() { return 42; }
	};

	static_assert(!THasToString<StructWithToStringWrongReturnType>::value, "Struct with wrong return type on ToString should not have ToString");

	struct SomeTestStruct
	{
	};

	static_assert(!THasToString<SomeTestStruct>::value, "Struct without ToString should not have ToString");
	static_assert(!THasOStream<SomeTestStruct>::value, "Struct without OStream operator should not have OStream");
} // namespace


namespace CQTestConvert
{
	template <typename T>
	inline FString ToString(const T& Input)
	{
		if constexpr (THasToString<T>::value)
		{
			return Input.ToString();
		}
		else if constexpr (THasOStream<T>::value)
		{
			PrintStream stream;
			stream << Input;
			return stream.ToString();
		}
		else if constexpr (std::is_floating_point_v<T>)
		{
			// Fallback for floating point type
			return FString::Printf(TEXT("%f"), Input);
		}
		else if constexpr (std::is_enum_v<T>)
		{
			// Fallback for enum type
			return ToString(static_cast<std::underlying_type_t<T>>(Input));
		}
		else if constexpr (std::is_pointer_v<T>)
		{
			// Fallback for pointer type
			return FString::Printf(TEXT("0x%p"), (void*)Input);
		}
		else
		{
			ensureMsgf(false, TEXT("Did not find ToString, ostream operator, or CQTestConvert::ToString template specialization for provided type. Cast provided type to something else, or see CqTestConvertTests.cpp for examples"));
			return TEXT("value");
		}
	}

	template <>
	inline FString ToString(const FString& Input)
	{
		return Input;
	}

	template<typename TElement, typename TAllocator>
	inline FString ToString(const TArray<TElement, TAllocator>& Input)
	{
		FString result = FString::JoinBy(Input, TEXT(", "), [](const TElement& e) {
			return CQTestConvert::ToString(e);
		});
		return TEXT("[") + result + TEXT("]");
	}

	template <typename TElement, typename TAllocator>
	inline FString ToString(const TSet<TElement, TAllocator>& Input)
	{
		FString result = FString::JoinBy(Input, TEXT(", "), [](const TElement& e) {
			return CQTestConvert::ToString(e);
		});
		return TEXT("[") + result + TEXT("]");
	}

	template<typename TKey, typename TValue>
	inline FString ToString(const TMap<TKey, TValue>& Input)
	{
		return FString::JoinBy(Input, TEXT(", "), [](const auto& kvp) {
			return TEXT("{") + CQTestConvert::ToString(kvp.Key) + TEXT(":") + CQTestConvert::ToString(kvp.Value) + TEXT("}");
		});
	}

	template<typename TInnerType>
	inline FString ToString(const TOptional<TInnerType>& Input)
	{
		if (Input.IsSet())
		{
			return ToString(Input.GetValue());
		}
		else
		{
			return TEXT("[NotSet]");
		}
	}
}
