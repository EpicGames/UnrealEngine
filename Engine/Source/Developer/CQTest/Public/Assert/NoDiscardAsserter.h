// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"

struct FNoDiscardAsserter
{
protected:
	FAutomationTestBase& TestRunner;

public:
	explicit FNoDiscardAsserter(FAutomationTestBase& TestRunner)
		: TestRunner(TestRunner) {}

	void ExpectError(FString Error, int32 Count = 1);
	void ExpectErrorRegex(FString Error, int32 Count = 1);

	void Fail(FString error);

	[[nodiscard]] bool IsTrue(bool Condition);
	[[nodiscard]] bool IsTrue(bool Condition, const char* FailureMessage);
	[[nodiscard]] bool IsTrue(bool Condition, const TCHAR* FailureMessage);
	[[nodiscard]] bool IsTrue(bool Condition, const FString& FailureMessage);

	[[nodiscard]] bool IsFalse(bool Condition);
	[[nodiscard]] bool IsFalse(bool Condition, const char* FailureMessage);
	[[nodiscard]] bool IsFalse(bool Condition, const TCHAR* FailureMessage);
	[[nodiscard]] bool IsFalse(bool Condition, const FString& FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNull(const T& Ptr);

	template <typename T>
	[[nodiscard]] bool IsNull(const T& Ptr, const char* FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNull(const T& Ptr, const TCHAR* FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNull(const T& Ptr, const FString& FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNotNull(const T& Ptr);

	template <typename T>
	[[nodiscard]] bool IsNotNull(const T& Ptr, const char* FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNotNull(const T& Ptr, const TCHAR* FailureMessage);

	template <typename T>
	[[nodiscard]] bool IsNotNull(const T& Ptr, const FString& FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreEqual(const TExpected& Expected, const TActual& Actual);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreEqual(const TExpected& Expected, const TActual& Actual, const char* FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreEqual(const TExpected& Expected, const TActual& Actual, const TCHAR* FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreEqual(const TExpected& Expected, const TActual& Actual, const FString& FailureMessage);

	[[nodiscard]] bool AreEqualIgnoreCase(const FString& Expected, const FString& Actual);

	[[nodiscard]] bool AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const char* FailureMessage);

	[[nodiscard]] bool AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const TCHAR* FailureMessage);

	[[nodiscard]] bool AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const FString& FailureMessage);

	[[nodiscard]] bool AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual);

	[[nodiscard]] bool AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const char* FailureMessage);

	[[nodiscard]] bool AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const TCHAR* FailureMessage);

	[[nodiscard]] bool AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const FString& FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreNotEqual(const TExpected& Expected, const TActual& Actual);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreNotEqual(const TExpected& Expected, const TActual& Actual, const char* FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreNotEqual(const TExpected& Expected, const TActual& Actual, const TCHAR* FailureMessage);

	template <typename TExpected, typename TActual>
	[[nodiscard]] bool AreNotEqual(const TExpected& Expected, const TActual& Actual, const FString& FailureMessage);

	template <typename TExpected, typename TActual, typename TEpsilon>
	[[nodiscard]] bool IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon);

	template <typename TExpected, typename TActual, typename TEpsilon>
	[[nodiscard]] bool IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const char* FailureMessage);

	template <typename TExpected, typename TActual, typename TEpsilon>
	[[nodiscard]] bool IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const TCHAR* FailureMessage);

	template <typename TExpected, typename TActual, typename TEpsilon>
	[[nodiscard]] bool IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const FString& FailureMessage);

};

#include "Assert/NoDiscardAsserter.inl"