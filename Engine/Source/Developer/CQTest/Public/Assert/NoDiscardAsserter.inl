// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/Regex.h"
#include "Assert/CQTestConvert.h"
#include "Assert/CQTestCondition.h"

inline void FNoDiscardAsserter::ExpectError(FString Error, int32 Count)
{
	TestRunner.AddExpectedError(Error, EAutomationExpectedErrorFlags::Contains, Count, false);
}

inline void FNoDiscardAsserter::ExpectErrorRegex(FString Error, int32 Count)
{
	TestRunner.AddExpectedError(Error, EAutomationExpectedErrorFlags::Contains, Count);
}

FORCEINLINE void FNoDiscardAsserter::Fail(FString Error)
{
	TestRunner.AddError(Error);
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsTrue(bool Condition)
{
	return IsTrue(Condition, TEXT("Expected condition to be true."));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsTrue(bool Condition, const char* FailureMessage)
{
	return IsTrue(Condition, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsTrue(bool Condition, const TCHAR* FailureMessage)
{
	return IsTrue(Condition, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsTrue(bool Condition, const FString& FailureMessage)
{
	if (!Condition)
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsFalse(bool Condition)
{
	return IsFalse(Condition, TEXT("Expected condition to be false."));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsFalse(bool Condition, const char* FailureMessage)
{
	return IsFalse(Condition, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsFalse(bool Condition, const TCHAR* FailureMessage)
{
	return IsFalse(Condition, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsFalse(bool Condition, const FString& FailureMessage)
{
	if (Condition)
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNull(const T& Ptr)
{
	return IsNull(Ptr, TEXT("Expected pointer to be Null"));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNull(const T& Ptr, const char* FailureMessage)
{
	return IsNull(Ptr, FString(FailureMessage));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNull(const T& Ptr, const TCHAR* FailureMessage)
{
	return IsNull(Ptr, FString(FailureMessage));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNull(const T& Ptr, const FString& FailureMessage)
{
	if (Ptr != nullptr)
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNotNull(const T& Ptr)
{
	return IsNotNull(Ptr, TEXT("Expected pointer to be not Null"));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNotNull(const T& Ptr, const char* FailureMessage)
{
	return IsNotNull(Ptr, FString(FailureMessage));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNotNull(const T& Ptr, const TCHAR* FailureMessage)
{
	return IsNotNull(Ptr, FString(FailureMessage));
}

template <typename T>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNotNull(const T& Ptr, const FString& FailureMessage)
{
	if (Ptr == nullptr)
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqual(const TExpected& Expected, const TActual& Actual)
{
	return AreEqual(Expected, Actual, FString::Printf(TEXT("Expected %s to equal %s."), *CQTestConvert::ToString(Expected), *CQTestConvert::ToString(Actual)));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqual(const TExpected& Expected, const TActual& Actual, const char* FailureMessage)
{
	return AreEqual(Expected, Actual, FString(FailureMessage));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqual(const TExpected& Expected, const TActual& Actual, const TCHAR* FailureMessage)
{
	return AreEqual(Expected, Actual, FString(FailureMessage));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqual(const TExpected& Expected, const TActual& Actual, const FString& FailureMessage)
{
	static_assert(!std::is_floating_point<TExpected>::value && !std::is_floating_point<TActual>::value,
		"Floating point types should not be compared for strict equality. Instead use IsNear() to account for floating point error. To directly compare without accounting for error, use IsTrue(A == B).");

	if (CQTestCondition::IsNotEqual(Expected, Actual))
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqualIgnoreCase(const FString& Expected, const FString& Actual)
{
	return AreEqualIgnoreCase(Expected, Actual, FString::Printf(TEXT("Expected %s to equal %s ignoring case."), *Expected, *Actual));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const char* FailureMessage)
{
	return AreEqualIgnoreCase(Expected, Actual, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const TCHAR* FailureMessage)
{
	return AreEqualIgnoreCase(Expected, Actual, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreEqualIgnoreCase(const FString& Expected, const FString& Actual, const FString& FailureMessage)
{
	if (CQTestCondition::IsNotEqualIgnoreCase(Expected, Actual))
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqual(const TExpected& Expected, const TActual& Actual)
{
	return AreNotEqual(Expected, Actual, FString::Printf(TEXT("Expected %s to not equal %s."), *CQTestConvert::ToString(Expected), *CQTestConvert::ToString(Actual)));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqual(const TExpected& Expected, const TActual& Actual, const char* FailureMessage)
{
	return AreNotEqual(Expected, Actual, FString(FailureMessage));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqual(const TExpected& Expected, const TActual& Actual, const TCHAR* FailureMessage)
{
	return AreNotEqual(Expected, Actual, FString(FailureMessage));
}

template <typename TExpected, typename TActual>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqual(const TExpected& Expected, const TActual& Actual, const FString& FailureMessage)
{
	static_assert(!std::is_floating_point<TExpected>::value && !std::is_floating_point<TActual>::value,
		"Floating point types should not be compared for strict equality. Instead use !IsNear() to account for floating point error. To directly compare without accounting for error, use IsTrue(A != B).");

	if (CQTestCondition::IsEqual(Expected, Actual))
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual)
{
	return AreNotEqualIgnoreCase(Expected, Actual, FString::Printf(TEXT("Expected %s to not equal %s ignoring case."), *Expected, *Actual));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const char* FailureMessage)
{
	return AreNotEqualIgnoreCase(Expected, Actual, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const TCHAR* FailureMessage)
{
	return AreNotEqualIgnoreCase(Expected, Actual, FString(FailureMessage));
}

[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::AreNotEqualIgnoreCase(const FString& Expected, const FString& Actual, const FString& FailureMessage)
{
	if (CQTestCondition::IsEqualIgnoreCase(Expected, Actual))
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}

template <typename TExpected, typename TActual, typename TEpsilon>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon)
{
	return IsNear(Expected, Actual, Epsilon, FString::Printf(TEXT("Expected %s to be near %s (within %s)."), *CQTestConvert::ToString(Expected), *CQTestConvert::ToString(Actual), *CQTestConvert::ToString(Epsilon)));
}

template <typename TExpected, typename TActual, typename TEpsilon>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const char* FailureMessage)
{
	return IsNear(Expected, Actual, Epsilon, FString(FailureMessage));
}

template <typename TExpected, typename TActual, typename TEpsilon>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const TCHAR* FailureMessage)
{
	return IsNear(Expected, Actual, Epsilon, FString(FailureMessage));
}

template <typename TExpected, typename TActual, typename TEpsilon>
[[nodiscard]] FORCEINLINE bool FNoDiscardAsserter::IsNear(TExpected Expected, TActual Actual, TEpsilon Epsilon, const FString& FailureMessage)
{
	if (CQTestCondition::IsNotNearlyEqual(Expected, Actual, Epsilon))
	{
		Fail(FailureMessage);
		return false;
	}
	return true;
}
