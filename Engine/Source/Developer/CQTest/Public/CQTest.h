// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"

#include "Misc/AutomationTest.h"
#include "HAL/Platform.h"

#include "Commands/TestCommands.h"
#include "Commands/TestCommandBuilder.h"

#include "Assert/NoDiscardAsserter.h"

/*
//Example boiler plate

#include "CQTest.h"
//TEST_CLASS(MyFixtureName, GenerateTestDirectory)
TEST_CLASS(MyFixtureName, "Game.Example")
{
	//static variables setup in BEFORE_ALL and torn down in AFTER_ALL

	//Member variables shared between tests

	BEFORE_ALL()
	{
		//static method
		//delete if empty
	}

	BEFORE_EACH()
	{
		//delete if empty
	}

	AFTER_EACH()
	{
		//delete if empty
	}

	AFTER_ALL()
	{
		//static method
		//delete if empty
	}

	TEST_METHOD(When_Given_Expect)
	{
		ASSERT_THAT(IsTrue(false));
	}
};
*/

namespace TestDirectoryGenerator
{
	CQTEST_API FString Generate(const FString& Filename);
}

enum class ECQTestSuppressLogBehavior
{
	Default,
	True,
	False
};

CQTEST_API extern const FString GenerateTestDirectory;
CQTEST_API extern const FString DefaultTags;
static constexpr EAutomationTestFlags DefaultFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter;

template <typename T>
concept HasBeforeAll = requires(T t) { { T::BeforeAll(FString()) }; };

template <typename T>
concept HasAfterAll = requires(T t) { { T::AfterAll(FString()) }; };

using BeforeAfterAllFunc = void(*)(const FString&);

template <typename InAsserterType>
struct TBaseTest
{
	using AsserterType = InAsserterType;

	TBaseTest(FAutomationTestBase& TestRunner, bool bInitializing);
	TBaseTest(const TBaseTest<AsserterType>& other) = delete;
	TBaseTest& operator=(const TBaseTest<AsserterType>& rhs) = delete;

	virtual ~TBaseTest() = default;

	virtual void Setup() {}

	virtual void TearDown() {}

	// The framework expects you to pass a 'new' command, which it will own and destroy
	void AddCommand(IAutomationLatentCommand* Cmd);
	void AddCommand(TSharedPtr<IAutomationLatentCommand> Cmd);

	/**
	 * Adds an error message to this test
	 *
	 * @param	InError	Error message to add to this test
	 */
	void AddError(const FString& InError) const;

	/**
	 * Adds an error message to this test if the condition is false
	 *
	 * @param   bCondition	The condition to validate.
	 * @param   InError		Error message to add to this test
	 * @return	False if there was an error
	 */
	bool AddErrorIfFalse(bool bCondition, const FString& InError) const;

	/**
	 * Adds a warning to this test
	 *
	 * @param	InWarning	Warning message to add to this test
	 */
	void AddWarning(const FString& InWarning) const;

	/**
	 * Adds a log item to this test
	 *
	 * @param	InLogItem	Log item to add to this test
	 */
	void AddInfo(const FString& InLogItem) const;

	virtual void RunTest(const FString& MethodName) = 0;

	bool bInitializing{ true };

	BeforeAfterAllFunc BeforeAllFunc = nullptr;
	BeforeAfterAllFunc AfterAllFunc = nullptr;

	FAutomationTestBase& TestRunner;
	AsserterType Assert;
	FTestCommandBuilder TestCommandBuilder;
};

template <typename InAsserterType>
struct TTestRunner;

template <typename InAsserterType>
using TTestInstanceGenerator = TUniquePtr<TBaseTest<InAsserterType>>(*)(TTestRunner<InAsserterType>&);

template <typename InAsserterType>
struct TTestRunner : public FAutomationTestBase
{
	using AsserterType = InAsserterType;

	TTestRunner(FString Name, int32 LineNumber, const char* FileName, FString TestDir, EAutomationTestFlags TestFlags, TTestInstanceGenerator<AsserterType> Factory, FString TestTags);
	~TTestRunner();

	EAutomationTestFlags GetTestFlags() const override;
	FString GetTestSourceFileName() const override;
	int32 GetTestSourceFileLine() const override;
	int32 GetTestSourceFileLine(const FString& Name) const override;
	bool SuppressLogWarnings() override;
	bool SuppressLogErrors() override;
	void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override;

	bool RunTest(const FString& RequestedTest) override;

	virtual void SetSuppressLogWarnings(ECQTestSuppressLogBehavior Behavior = ECQTestSuppressLogBehavior::True);
	virtual void SetSuppressLogErrors(ECQTestSuppressLogBehavior Behavior = ECQTestSuppressLogBehavior::True);

	const FString GetTestTags() const { return TestTags; }
	const FString GetTestMethodTags(const FString& MethodName) const
	{
		FString FullMethodName = FString::Printf(TEXT("%s.%s"), *GetBeautifiedTestName(), *MethodName);
		return FAutomationTestFramework::Get().GetTagsForAutomationTest(FullMethodName);
	}

	FString GetBeautifiedTestName() const override;

	int32 LineNumber = 0;
	FString FileName;
	FString TestDir;
	EAutomationTestFlags TestFlags{ DefaultFlags };
	bool bInitializing{ true };

	TUniquePtr<TBaseTest<AsserterType>> CurrentTestPtr;
	TArray<FString> TestNames;
	TMap<FString, int32> TestLineNumbers{};
	TTestInstanceGenerator<AsserterType> TestInstanceFactory;
	FDelegateHandle BeforeAllDelegate{};
	FDelegateHandle AfterAllDelegate{};

	TTestRunner() = delete;

protected:
	uint32 GetRequiredDeviceNum() const override;

	ECQTestSuppressLogBehavior SuppressLogWarningsBehavior{ ECQTestSuppressLogBehavior::Default };
	ECQTestSuppressLogBehavior SuppressLogErrorsBehavior{ ECQTestSuppressLogBehavior::Default };

private:
	FString TestTags;
};

template <typename Derived, typename AsserterType>
struct TTest : TBaseTest<AsserterType>
{
	using TestMethod = void (Derived::*)();
	using DerivedType = Derived;

	TTest()
		: TBaseTest<AsserterType>(*Derived::TestRunner, Derived::TestRunner->bInitializing)
	{
		if constexpr (HasBeforeAll<Derived>)
		{
			this->BeforeAllFunc = Derived::BeforeAll;
		}
		if constexpr (HasAfterAll<Derived>)
		{
			this->AfterAllFunc = Derived::AfterAll;
		}
	}

	void RunTest(const FString& TestName) override;

	struct FFunctionRegistrar
	{
		FFunctionRegistrar(FString Name, TestMethod Func, int32 LineNumber, FString TestTags)
		{
			// Only register the test method if not already registered
			if (!DerivedType::Methods.Contains(Name))
			{
				DerivedType::TestRunner->TestNames.Add(Name);
				DerivedType::Methods.Add(Name, Func);
				DerivedType::TestRunner->TestLineNumbers.Add(Name, LineNumber);

				FString CommonTestTags = DerivedType::TestRunner->GetTestTags();
				FString CompleteTags = FString::Printf(TEXT("%s%s"), *CommonTestTags, *TestTags);
				if (!CompleteTags.IsEmpty())
				{
					FString FullName = FString::Printf(TEXT("%s.%s"), *DerivedType::TestRunner->GetBeautifiedTestName(), *Name);
					bool Registered = FAutomationTestFramework::Get().RegisterAutomationTestTags(FullName, CompleteTags);
					if (Registered)
					{
						MethodTags = CompleteTags;
					}
				}
			}
		}

		const FString GetTestMethodTags() const { return MethodTags; }
	private:
		FString MethodTags;
	};

	static TUniquePtr<TBaseTest<AsserterType>> CreateTestClass(TTestRunner<AsserterType>& TestRunnerInstance)
	{
		DerivedType::TestRunner = &TestRunnerInstance;
		return MakeUnique<DerivedType>();
	}

	inline static TMap<FString, TestMethod> Methods{};
	inline static TTestRunner<AsserterType>* TestRunner{ nullptr };
};

#if WITH_AUTOMATION_WORKER

#define _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags, _TestTags)                                              \
	struct _ClassName;                                                                                                                            \
	struct F##_ClassName##_Runner : public TTestRunner<_AsserterType>                                                                             \
	{                                                                                                                                             \
		F##_ClassName##_Runner()                                                                                                                  \
			: TTestRunner(#_ClassName, __LINE__, __FILE__, _TestDir, _TestFlags, TTest<_ClassName, _AsserterType>::CreateTestClass, _TestTags) {  \
				static_assert(  !!((_TestFlags) & EAutomationTestFlags_ApplicationContextMask),                                                   \
								"CQTest has no application flag and will not run. See AutomationTest.h.");                                        \
				static_assert(  !!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::SmokeFilter) ||                      \
								!!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::EngineFilter) ||                     \
								!!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::ProductFilter) ||                    \
								!!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::PerfFilter) ||                       \
								!!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::StressFilter) ||                     \
								!!(((_TestFlags) & EAutomationTestFlags_FilterMask) == EAutomationTestFlags::NegativeFilter),                     \
								"All CQTests must have exactly 1 filter type specified. See AutomationTest.h.");                                  \
		}                                                                                                                                         \
	};                                                                                                                                            \
	F##_ClassName##_Runner _ClassName##_RunnerInstance;                                                                                           \
	struct _ClassName : public _BaseClass<_ClassName, _AsserterType>

#define _TEST_CLASS_IMPL(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags)                  \
		_TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags, DefaultTags)

#define TEST_METHOD_WITH_TAGS(_MethodName, _TestTags)                                                             \
	FFunctionRegistrar reg##_MethodName{ FString(#_MethodName), &DerivedType::_MethodName, __LINE__, _TestTags }; \
	void _MethodName()

#define TEST_METHOD(_MethodName) TEST_METHOD_WITH_TAGS(_MethodName, DefaultTags)

#else
#define _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags, _TestTags) \
	struct _ClassName : public _BaseClass<_ClassName, _AsserterType>

#define _TEST_CLASS_IMPL(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags) _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, _AsserterType, _TestFlags, DefaultTags)

#define TEST_METHOD_WITH_TAGS(_MethodName, _TestTags) void _MethodName()
#define TEST_METHOD(_MethodName) TEST_METHOD_WITH_TAGS(_MethodName, DefaultTags)
#endif

#define TEST_CLASS_WITH_ASSERTS(_ClassName, _TestDir, _AsserterType) _TEST_CLASS_IMPL(_ClassName, _TestDir, TTest, _AsserterType, DefaultFlags)
#define TEST_CLASS_WITH_ASSERTS_AND_TAGS(_ClassName, _TestDir, _AsserterType, _TestTags) _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, TTest, _AsserterType, DefaultFlags, _TestTags)
#define TEST_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_ASSERTS(_ClassName, _TestDir, FNoDiscardAsserter)
#define TEST_CLASS_WITH_TAGS(_ClassName, _TestDir, _TestTags) TEST_CLASS_WITH_ASSERTS_AND_TAGS(_ClassName, _TestDir, FNoDiscardAsserter, _TestTags)
#define TEST_CLASS_WITH_BASE(_ClassName, _TestDir, _BaseClass) _TEST_CLASS_IMPL(_ClassName, _TestDir, _BaseClass, FNoDiscardAsserter, DefaultFlags)
#define TEST_CLASS_WITH_BASE_AND_TAGS(_ClassName, _TestDir, _BaseClass, _TestTags) _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, FNoDiscardAsserter, DefaultFlags, _TestTags)
#define TEST_CLASS_WITH_BASE_AND_ASSERTS(_ClassName, _TestDir, _BaseClass, _AsserterType) _TEST_CLASS_IMPL(_ClassName, _TestDir, _BaseClass, _AsserterType, DefaultFlags)
#define TEST_CLASS_WITH_FLAGS(_ClassName, _TestDir, _Flags) _TEST_CLASS_IMPL(_ClassName, _TestDir, TTest, FNoDiscardAsserter, _Flags)
#define TEST_CLASS_WITH_FLAGS_AND_TAGS(_ClassName, _TestDir, _Flags, _TestTags) _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, TTest, FNoDiscardAsserter, _Flags, _TestTags)
#define TEST_CLASS_WITH_BASE_AND_FLAGS(_ClassName, _TestDir, _BaseClass, _Flags) _TEST_CLASS_IMPL(_ClassName, _TestDir, _BaseClass, FNoDiscardAsserter, _Flags)
#define TEST_CLASS_WITH_BASE_AND_FLAGS_AND_TAGS(_ClassName, _TestDir, _BaseClass, _Flags, _TestTags) _TEST_CLASS_IMPL_EXT(_ClassName, _TestDir, _BaseClass, FNoDiscardAsserter, _Flags, _TestTags)

#define TEST(_TestName, _TestDir)        \
	TEST_CLASS(_TestName, _TestDir)      \
	{                                    \
		TEST_METHOD(_TestName##_Method); \
	};                                   \
	void _TestName::_TestName##_Method()

#define TEST_WITH_TAGS(_TestName, _TestDir, _TestTags)   \
	TEST_CLASS_WITH_TAGS(_TestName, _TestDir, _TestTags) \
	{                                                    \
		TEST_METHOD(_TestName##_Method);                 \
	};                                                   \
	void _TestName::_TestName##_Method()

#define ASSERT_THAT(_assertion) if (!this->Assert._assertion) {return;}
#define ASSERT_FAIL(_Msg) this->Assert.Fail(_Msg); return;

#define BEFORE_ALL() static void BeforeAll(const FString&)
#define BEFORE_EACH() virtual void Setup() override
#define AFTER_EACH() virtual void TearDown() override
#define AFTER_ALL() static void AfterAll(const FString&)

#include "Impl/CQTest.inl"
