// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettingsBackedByCVars.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigUtilities.h"

#include "CQTestSettings.generated.h"

#define UE_API CQTEST_API

struct FScopedTestEnvironment;

namespace CQTestConsoleVariables 
{
inline static constexpr float CommandTimeout = 10.0f;
constexpr auto CommandTimeoutName = TEXT("TestFramework.CQTest.CommandTimeout");

inline static constexpr float NetworkTimeout = 30.0f;
constexpr auto NetworkTimeoutName = TEXT("TestFramework.CQTest.CommandTimeout.Network");

inline static constexpr float MapTestTimeout = 30.0f;
constexpr auto MapTestTimeoutName = TEXT("TestFramework.CQTest.CommandTimeout.MapTest");
} // namespace CQTestConsoleVariables

/** Implements per project Engine settings for the CQTest plugin. */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "CQ Test Settings"))
class UCQTestSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()

public:
	void PostInitProperties() override
	{
		if (IsTemplate())
		{
			// We want the .ini file to have precedence over the CVar constructor, so we apply the ini to the CVar before following the regular UDeveloperSettingsBackedByCVars flow
			UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/CQTest.CQTestSettings"), *GEngineIni, ECVF_SetByProjectSetting);
		}

		Super::PostInitProperties();
	}

	/** Timeout for WaitUntil latent actions. */
	UPROPERTY(config, EditAnywhere, Category = "Test Settings", meta = (ConsoleVariable = "TestFramework.CQTest.CommandTimeout"))
	float CommandTimeout = CQTestConsoleVariables::CommandTimeout;

	/** Timeout for WaitUntil latent actions from the PIENetworkComponent. */
	UPROPERTY(config, EditAnywhere, Category = "Test Settings", meta = (ConsoleVariable = "TestFramework.CQTest.CommandTimeout.Network"))
	float NetworkTimeout = CQTestConsoleVariables::NetworkTimeout;

	/** Timeout for `FMapTestSpawner::AddWaitUntilLoadedCommand` latent action used during UWorld loading. */
	UPROPERTY(config, EditAnywhere, Category = "Test Settings", meta = (ConsoleVariable = "TestFramework.CQTest.CommandTimeout.MapTest"))
	float MapTestTimeout = CQTestConsoleVariables::MapTestTimeout;

	/**
	 * Sets the duration for all available timeouts
	 *
	 * @param Duration - Duration of the timeout.
	 * 
	 * @return shared pointer to the FScopedTestEnvironment singleton instance
	 *
	 * @note Console Variable will reset back to the value prior to getting set once the FScopedTestEnvironment resets or gets destructed.
	 */
	[[nodiscard]] static UE_API TSharedPtr<FScopedTestEnvironment> SetTestClassTimeouts(FTimespan Duration);
};

#undef UE_API
