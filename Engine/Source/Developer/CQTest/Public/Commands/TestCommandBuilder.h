// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Reverse.h"
#include "Misc/AutomationTest.h"
#include "Commands/TestCommands.h"

class FTestCommandBuilder
{
public:
	FTestCommandBuilder(FAutomationTestBase& InTestRunner)
		: TestRunner(InTestRunner) {}

	~FTestCommandBuilder()
	{
		checkf(CommandQueue.IsEmpty(), TEXT("Adding latent actions from within latent actions is currently unsupported."));
	}

	FTestCommandBuilder& Do(const TCHAR* Description, TFunction<void()> Action)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<FExecute>(TestRunner, Action, Description));
		}
		return *this;
	}

	FTestCommandBuilder& Do(TFunction<void()> Action)
	{
		return Do(nullptr, Action);
	}

	FTestCommandBuilder& Then(TFunction<void()> Action)
	{
		return Do(Action);
	}

	FTestCommandBuilder& Then(const TCHAR* Description, TFunction<void()> Action)
	{
		return Do(Description, Action);
	}

	FTestCommandBuilder& Until(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<FWaitUntil>(TestRunner, Query, Timeout, Description));
		}
		return *this;
	}

	FTestCommandBuilder& Until(TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return Until(nullptr, Query, Timeout);
	}

	template <typename ResultType>
	FTestCommandBuilder& DoAsync(const TCHAR* Description, TFunction<TAsyncResult<ResultType>()> AsyncAction, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<TAsyncExecute<ResultType>>(TestRunner, MoveTemp(AsyncAction), Timeout, Description));
		}
		return *this;
	}

	template <typename ResultType>
	FTestCommandBuilder& DoAsync(TFunction<TAsyncResult<ResultType>()> AsyncAction, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(nullptr, MoveTemp(AsyncAction), Timeout);
	}

	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type DoAsync(
		const TCHAR* Description,
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<void(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<TAsyncExecute<ResultType, FExecute>>(TestRunner, MoveTemp(AsyncAction), Timeout, Description, MoveTemp(ResultCallback)));
		}
		return *this;
	}

	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type DoAsync(
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<void(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(nullptr, MoveTemp(AsyncAction), MoveTemp(ResultCallback), Timeout);
	}

	template <typename ResultType>
	FTestCommandBuilder& ThenAsync(const TCHAR* Description, TFunction<TAsyncResult<ResultType>()> AsyncAction, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(Description, MoveTemp(AsyncAction), Timeout);
	}

	template <typename ResultType>
	FTestCommandBuilder& ThenAsync(TFunction<TAsyncResult<ResultType>()> AsyncAction, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(nullptr, MoveTemp(AsyncAction), Timeout);
	}

	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type ThenAsync(
		const TCHAR* Description,
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<void(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(Description, MoveTemp(AsyncAction), MoveTemp(ResultCallback), Timeout);
	}
	
	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type ThenAsync(
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<void(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return DoAsync<ResultType>(nullptr, MoveTemp(AsyncAction), MoveTemp(ResultCallback), Timeout);
	}

	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type UntilAsync(
		const TCHAR* Description,
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<bool(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> AsyncActionTimeout = CQTest::DefaultTimeout,
		TOptional<FTimespan> ConditionTimeout = CQTest::DefaultTimeout)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<TAsyncExecute<ResultType, FWaitUntil>>(TestRunner, MoveTemp(AsyncAction), AsyncActionTimeout, Description, MoveTemp(ResultCallback), ConditionTimeout));
		}
		return *this;
	}

	template <typename ResultType>
	typename TEnableIf<!std::is_void_v<ResultType>, FTestCommandBuilder&>::Type UntilAsync(
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TFunction<bool(typename TAsyncResultCallbackArg<ResultType>::Type)> ResultCallback,
		TOptional<FTimespan> AsyncActionTimeout = CQTest::DefaultTimeout,
		TOptional<FTimespan> ConditionTimeout = CQTest::DefaultTimeout)
	{
		return UntilAsync<ResultType>(nullptr, MoveTemp(AsyncAction), MoveTemp(ResultCallback), AsyncActionTimeout, ConditionTimeout);
	}

	FTestCommandBuilder& StartWhen(TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return Until(Query, Timeout);
	}

	FTestCommandBuilder& StartWhen(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
	{
		return Until(Description, Query, Timeout);
	}

	/** Note that using a timed-wait can introduce test flakiness due to variable runtimes. Please consider using `Until` and waiting until something happens instead. */
	FTestCommandBuilder& WaitDelay(FTimespan Timeout)
	{
		return WaitDelay(nullptr, Timeout);
	}

	/** Note that using a timed-wait can introduce test flakiness due to variable runtimes. Please consider using `Until` and waiting until something happens instead. */
	FTestCommandBuilder& WaitDelay(const TCHAR* Description, FTimespan Timeout)
	{
		if (!TestRunner.HasAnyErrors())
		{
			CommandQueue.Add(MakeShared<FWaitDelay>(TestRunner, Timeout, Description));
		}
		return *this;
	}

	FTestCommandBuilder& OnTearDown(const TCHAR* Description, TFunction<void()> Action)
	{
		if (!TestRunner.HasAnyErrors())
		{
			TearDownQueue.Add(MakeShared<FExecute>(TestRunner, Action, Description, ECQTestFailureBehavior::Run));
		}
		return *this;
	}

	FTestCommandBuilder& OnTearDown(TFunction<void()> Action)
	{
		return OnTearDown(nullptr, Action);
	}

	FTestCommandBuilder& CleanUpWith(const TCHAR* Description, TFunction<void()> Action)
	{
		return OnTearDown(Description, Action);
	}

	FTestCommandBuilder& CleanUpWith(TFunction<void()> Action)
	{
		return OnTearDown(nullptr, Action);
	}

	TSharedPtr<IAutomationLatentCommand> Build()
	{
		return BuildQueue(CommandQueue);
	}

	TSharedPtr<IAutomationLatentCommand> BuildTearDown()
	{
		// Last in, first out
		Algo::Reverse(TearDownQueue);
		return BuildQueue(TearDownQueue);
	}

private:
	TSharedPtr<IAutomationLatentCommand> BuildQueue(TArray<TSharedPtr<IAutomationLatentCommand>>& Queue)
	{
		TSharedPtr<IAutomationLatentCommand> Result = nullptr;
		if (Queue.Num() == 0)
		{
			return Result;
		}
		else if (Queue.Num() == 1)
		{
			Result = Queue[0];
		}
		else
		{
			Result = MakeShared<FRunSequence>(Queue);
		}

		Queue.Empty();
		return Result;
	}

protected:
	TArray<TSharedPtr<IAutomationLatentCommand>> CommandQueue{};
	TArray<TSharedPtr<IAutomationLatentCommand>> TearDownQueue{};
	FAutomationTestBase& TestRunner;

	template<typename Asserter>
	friend struct TBaseTest;
};