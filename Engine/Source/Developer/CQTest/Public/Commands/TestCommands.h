// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "Async/AsyncResult.h"

#define UE_API CQTEST_API

namespace CQTest
{
inline static const TOptional<FTimespan> DefaultTimeout = NullOpt;
}

/** Latent Command that waits until the Query evaluates to `true` or the timeout has exceeded. */
class FWaitUntil : public IAutomationLatentCommand
{
public:
	FWaitUntil(FAutomationTestBase& InTestRunner, TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout, const TCHAR* InDescription = nullptr)
		: TestRunner(InTestRunner)
		, Query(MoveTemp(Query)) 
		, Timeout(MakeTimeout(Timeout))
		, Description(InDescription)
	{}

	UE_API bool Update() override;

	FAutomationTestBase& TestRunner;
	TFunction<bool()> Query;
	FTimespan Timeout;
	FDateTime StartTime;
	const TCHAR* Description;
	bool bHasTimerStarted = false;

private:
	UE_API FTimespan MakeTimeout(TOptional<FTimespan> InTimeout);
};

/** Latent Command that waits a set time frame.
 * Note that using a timed-wait can introduce test flakiness due to variable runtimes. Please consider using `FWaitUntil` and waiting until something happens instead.
 */
class FWaitDelay : public IAutomationLatentCommand
{
public:
	FWaitDelay(FAutomationTestBase& InTestRunner, FTimespan Timeout, const TCHAR* InDescription = nullptr)
		: TestRunner(InTestRunner)
		, Timeout(Timeout)
		, Description(InDescription)
	{}

	UE_API bool Update() override;

	FAutomationTestBase& TestRunner;
	FTimespan Timeout;
	FDateTime EndTime;
	const TCHAR* Description;
	bool bHasTimerStarted = false;
};

enum class ECQTestFailureBehavior
{
	Skip,
	Run
};

/** Latent Command which executes the provided function. */
class FExecute : public IAutomationLatentCommand
{
public:
	FExecute(FAutomationTestBase& InTestRunner, TFunction<void()> Func, const TCHAR* InDescription = nullptr, ECQTestFailureBehavior InFailureBehavior = ECQTestFailureBehavior::Skip)
		: TestRunner(InTestRunner)
		, Func(MoveTemp(Func))
		, Description(InDescription)
		, FailureBehavior(InFailureBehavior)
	{}

	UE_API bool Update() override;

	FAutomationTestBase& TestRunner;
	TFunction<void()> Func;
	const TCHAR* Description = nullptr;
	ECQTestFailureBehavior FailureBehavior;
};

/** Latent Command which manages and executes an array of latent commands. */
class FRunSequence : public IAutomationLatentCommand
{
public:
	FRunSequence(const TArray<TSharedPtr<IAutomationLatentCommand>>& ToAdd)
		: Commands(ToAdd)
	{
	}

	template <class... Cmds>
	FRunSequence(Cmds... Commands)
		: FRunSequence(TArray<TSharedPtr<IAutomationLatentCommand>>{ Commands... })
	{
	}

	UE_API void Append(TSharedPtr<IAutomationLatentCommand> ToAdd);
	UE_API void AppendAll(TArray < TSharedPtr<IAutomationLatentCommand>> ToAdd);
	UE_API void Prepend(TSharedPtr<IAutomationLatentCommand> ToAdd);

	bool IsEmpty() const
	{
		return Commands.IsEmpty();
	}

	UE_API bool Update() override;

	TArray<TSharedPtr<IAutomationLatentCommand>> Commands;
};

template <typename ResultType>
struct TAsyncResultCallbackArg
{
	using Type = decltype(std::declval<TAsyncResult<ResultType>>().GetFuture().Get());
};

template <typename ResultType, typename CommandType>
class TAsyncExecute;

/** This namespace provides a set of variables and functions intended for internal use within the TAsyncExecute class */
namespace AsyncExecuteDetails
{
	static const TCHAR* CreateInternalCommandDescription(TArray<FString>& OutDescriptions, const TCHAR* External, const TCHAR* Internal)
	{
		if (!External) { return nullptr; }
		OutDescriptions.Emplace(FString::Printf(TEXT("%s [%s]"), External, Internal));
		return *OutDescriptions.Last();
	}

	template <typename ResultType, typename CommandType>
	class TResultCommandFactory;

	/** This class provides a helper function for creating a FExecute command to handle the result of an async operation. */
	template <typename ResultType>
	class TResultCommandFactory<ResultType, FExecute>
	{
		static TSharedRef<IAutomationLatentCommand> Create(
			FAutomationTestBase& TestRunner,
			TAsyncResult<ResultType>& AsyncResult,
			TArray<FString>& OutDescriptions,
			const TCHAR* BaseDescription,
			TFunction<void(typename TAsyncResultCallbackArg<ResultType>::Type)>&& InFunc)
		{
			return MakeShared<FExecute>(
				TestRunner,
				[&Result = AsyncResult, Func = MoveTemp(InFunc)]() { return Func(Result.GetFuture().Get()); },
				CreateInternalCommandDescription(OutDescriptions, BaseDescription, TEXT("Handle result [FExecute]"))
			);
		}

		friend TAsyncExecute<ResultType, FExecute>;
	};
	
	/** This class provides a helper function for creating a FWaitUntil command to handle the result of an async operation. */
	template <typename ResultType>
	class TResultCommandFactory<ResultType, FWaitUntil>
	{
		static TSharedRef<IAutomationLatentCommand> Create(
			FAutomationTestBase& TestRunner,
			TAsyncResult<ResultType>& AsyncResult,
			TArray<FString>& OutDescriptions,
			const TCHAR* BaseDescription,
			TFunction<bool(typename TAsyncResultCallbackArg<ResultType>::Type)>&& InFunc,
			TOptional<FTimespan> Timeout = CQTest::DefaultTimeout)
		{
			return MakeShared<FWaitUntil>(
				TestRunner,
				[&Result = AsyncResult, Func = MoveTemp(InFunc)]() { return Func(Result.GetFuture().Get()); },
				Timeout,
				CreateInternalCommandDescription(OutDescriptions, BaseDescription, TEXT("Handle result [FWaitUntil]"))
			);
		}

		friend TAsyncExecute<ResultType, FWaitUntil>;
	};
};

/** Latent Command that executes an async action and optionally processes the result in a Latent Command of the specified type. */
template <typename ResultType, typename ResultCommandType = void>
class TAsyncExecute : public IAutomationLatentCommand
{
public:
	template<typename... ResultCommandArgs>
	TAsyncExecute(
		FAutomationTestBase& TestRunner,
		TFunction<TAsyncResult<ResultType>()> AsyncAction,
		TOptional<FTimespan> AsyncActionTimeout = CQTest::DefaultTimeout,
		const TCHAR* Description = nullptr,
		ResultCommandArgs&&... Args
	)
	{
		RunSequence.Append(MakeShared<FExecute>(
			TestRunner,
			[&Result = AsyncResult, Action = MoveTemp(AsyncAction)]() { Result = Action(); },
			AsyncExecuteDetails::CreateInternalCommandDescription(Descriptions, Description, TEXT("Execute async action"))
		));
		RunSequence.Append(MakeShared<FWaitUntil>(
			TestRunner,
			[&Result = AsyncResult]() { return Result.GetFuture().IsReady(); },
			AsyncActionTimeout,
			AsyncExecuteDetails::CreateInternalCommandDescription(Descriptions, Description, TEXT("Wait async result"))
		));
		if constexpr (!std::is_void_v<ResultCommandType>)
		{
			static_assert(!std::is_void_v<ResultType>,
				"The <ResultType> 'void' should not be used with <ResultCommandType>. "
				"Instead, consider using a standalone command for any subsequent actions.");

			RunSequence.Append(AsyncExecuteDetails::TResultCommandFactory<ResultType, ResultCommandType>::Create(
				TestRunner,
				AsyncResult,
				Descriptions,
				Description,
				std::forward<ResultCommandArgs>(Args)...)
			);
		}
	};

	bool Update() override { return RunSequence.Update(); }

private:
	FRunSequence RunSequence;
	TAsyncResult<ResultType> AsyncResult;
	TArray<FString> Descriptions;
};

#undef UE_API
