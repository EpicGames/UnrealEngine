// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TEST_EQ(Left, Right) TestRunner->TestEqual(TEXT(#Left "==" #Right), Left, Right)
#define TEST_TRUE(Condition) TestRunner->TestTrue(TEXT(#Condition), Condition)
#define ADD_ERROR(Error) TestRunner->AddError(Error)
#define TEST_ERROR(Error) TestRunner->AddExpectedError(Error)

#define DO(Cmd) TestCommandBuilder.Do([&]() { Cmd })
#define START_WHEN(Query) TestCommandBuilder.StartWhen([&]() { Query })
#define WAIT_DELAY(Duration) TestCommandBuilder.WaitDelay(Duration)
#define THEN(Cmd) Then([&]() { Cmd })
#define UNTIL(Query) Until([&]() { Query })