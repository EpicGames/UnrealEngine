// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpawnHelper.h"

#define UE_API CQTEST_API

/*
//Example boiler plate

#include "CQTest.h"
#include "Components/ActorTestSpawner.h"

TEST_CLASS(MyFixtureName, "ActorSpawner.Example")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(Spawn_BaseActor_DefaultActorSpawned)
	{
		AActor& Actor = Spawner.SpawnActor<AActor>();
		ASSERT_THAT(IsTrue(Actor.GetFName() == NAME_None));
	}

	TEST_METHOD(Spawn_BaseObject_DefaultObjectSpawned)
	{
		UObject& Object = Spawner.SpawnObject<UObject>();
		ASSERT_THAT(IsTrue(Object.GetFName() == NAME_None));
	}
};
*/

class UTestGameInstance;

/** Class for spawning actors in an ActorTest context(no PIE loaded) */
struct FActorTestSpawner : public FSpawnHelper
{
	FActorTestSpawner() = default;

	UE_API virtual ~FActorTestSpawner() override;

	/** Initialize the GameSubsystem for the test world. */
	UE_API void InitializeGameSubsystems();

	/** Returns a pointer to the test game instance. */
	UE_API UTestGameInstance* GetGameInstance();

protected:
	/** Returns a newly created world. */
	UE_API virtual UWorld* CreateWorld() override;

private:
	UTestGameInstance* GameInstance{ nullptr };
};

#undef UE_API
