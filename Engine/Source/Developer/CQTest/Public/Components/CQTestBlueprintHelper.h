// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "UObject/Class.h"

/*
//Example boiler plate

#if WITH_AUTOMATION_TESTS

#include "CQTest.h"
#include "Components/CQTestBlueprintHelper.h"

TEST_CLASS_WITH_FLAGS(MyFixtureName, "BlueprintHelper.Example", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)
{
	FCQTestBlueprintHelper BlueprintHelper;

	TEST_METHOD(Object_FromBlueprint_IsLoaded)
	{
		UClass* Class = BlueprintHelper.GetBlueprintClass(TEXT("/Package/Path/To/Asset"), TEXT("B_Asset"));
		ASSERT_THAT(IsNotNull(Class));

		UObject* Object = BlueprintHelper.FindDataBlueprint(TEXT("/Package/Path/To/Asset"), TEXT("B_Asset"));
		ASSERT_THAT(IsNotNull(Object));
	}
};

#endif // WITH_AUTOMATION_TESTS
*/

/** Utilities for working with Blueprint in C++ Tests */
struct UE_DEPRECATED(5.5, "Use Helpers/CQTestAssetHelper.h instead") CQTEST_API FCQTestBlueprintHelper
{
	/**
	 * Finds the UObject given a directory and asset name to find.
	 *
	 * @param Directory - Directory path where the asset resides.
	 * @param Name - Name of the asset.
	 * 
	 * @return pointer to the UObject that was found and loaded, returns nullptr if no object was found
	 * 
	 * @note Method uses a static cache to store information regarding directories and assets.
	 */
	UObject* FindDataBlueprint(const FString& Directory, const FString& Name);

	/**
	 * Gets the UClass of a UObject loaded from a given directory and asset name.
	 *
	 * @param Directory - Directory path where the asset resides.
	 * @param Name - Name of the asset.
	 * 
	 * @return pointer of the blueprint's generated class
	 *
	 * @note Method guarantees that the UClass returned is valid, will assert otherwise.
	 */
	UClass* GetBlueprintClass(const FString& Directory, const FString& Name);
};