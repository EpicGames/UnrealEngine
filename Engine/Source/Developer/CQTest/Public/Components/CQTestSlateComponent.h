// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

#include <atomic>

#define UE_API CQTEST_API

/*
//Example boiler plate

#include "CQTest.h"
#include "Components/CQTestSlateComponent.h"

TEST_CLASS(MyFixtureName, "Slate.Example")
{
	TUniquePtr<FCQTestSlateComponent> SlateComponent;

	BEFORE_EACH()
	{
		SlateComponent = MakeUnique<FCQTestSlateComponent>();
	}

	TEST_METHOD(HaveTicksElapsed_WaitUntil_EventuallyReturnsTrue)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]() { ASSERT_THAT(IsTrue(SlateComponent->GetTotalElapsedTicks() >= 3)); });
	}
};
*/

/** CQTest component for interacting with Slate */
class FCQTestSlateComponent
{
public:
	UE_API FCQTestSlateComponent();
	UE_API ~FCQTestSlateComponent();

	/**
	 * Initializes the ExpectedTick variable to be checked against the amount of times Slate has ticked.
	 * Will be used to check Slate has ticked the expected amount before confirming and resetting the ExpectedTick.
	 *
	 * @param Ticks - Number of ticks we're expecting to have elapsed
	 * 
	 * @return true when Slate has ticked the provided number of times.
	 * 
	 * @note Method is meant to be used as the only statement within a latent command that waits until the predicate is met, such as FWaitUntil, Until, or StartWhen methods.
	 */
	UE_API bool HaveTicksElapsed(uint32 Ticks);

	/** Returns the total elapsed ticks since creation. */
	uint32 GetTotalElapsedTicks()
	{
		return TickCounter;
	}

private:
	/**
	 * Called on Slate's Post Tick
	 *
	 * @param InDeltaTime - Elapsed time since the last tick. While the value is not being used directly, it is needed to register for Slate's delegate.
	 */
	UE_API void OnPostTick([[maybe_unused]] const float InDeltaTime);

	std::atomic<uint32> TickCounter{0};
	TOptional<uint32> ExpectedTick;
	FDelegateHandle TickDelegateHandle;
	TSharedPtr<struct FScopedTestEnvironment> TestEnvironment;
};

#undef UE_API
