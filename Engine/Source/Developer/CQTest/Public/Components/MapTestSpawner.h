// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpawnHelper.h"

#include "Commands/TestCommandBuilder.h"

/*
//Example boiler plate

#include "CQTest.h"
#include "Components/MapTestSpawner.h"

#if WITH_AUTOMATION_TESTS

TEST_CLASS(MyFixtureName, "MapSpawner.Example")
{
	TUniquePtr<FMapTestSpawner> Spawner;
	APawn* MyPawn;

	BEFORE_EACH()
	{
		Spawner = MakeUnique<FMapTestSpawner>(TEXT("/Package/Path/To/Map"), TEXT("MapName"));
		Spawner->AddWaitUntilLoadedCommand(TestRunner);
	}

	TEST_METHOD(PlayerPawn_Loaded_Found)
	{
		TestCommandBuilder
			.StartWhen([this]() {
				MyPawn = Spawner->FindFirstPlayerPawn();
				return nullptr != MyPawn;
			})
			.Then([this]() { ASSERT_THAT(IsNotNull(MyPawn)); });
	}
};

#endif // WITH_AUTOMATION_TESTS
*/

#if WITH_AUTOMATION_TESTS

#if WITH_EDITOR
#include "UnrealEdMisc.h"
#endif // WITH_EDITOR

/** Class for spawning Actors in a named Map / Level */
struct FMapTestSpawner : public FSpawnHelper
{
	/**
	 * Construct the MapTestSpawner.
	 *
	 * @param MapDirectory - The directory which the map resides in.
	 * @param MapName - Name of the map.
	 */
	FMapTestSpawner(const FString& MapDirectory, const FString& MapName)
		: MapDirectory(MapDirectory), MapName(MapName) {}

	/** Destructor */
	CQTEST_API ~FMapTestSpawner();

	/**
	 * Creates an instance of the MapTestSpawner with a temporary level ready for use.
	 * 
	 * @param InCommandBuilder - Test Command Builder used to assist with setup.
	 * 
	 * @return unique instance of the FMapTestSpawner, nullptr otherwise
	 */
	static CQTEST_API TUniquePtr<FMapTestSpawner> CreateFromTempLevel(FTestCommandBuilder& InCommandBuilder);

	/**
	 * Loads the map specified from the MapDirectory and MapName to be prepared for the test.
	 *
	 * @param TestRunner - TestRunner used to send the latent command needed for map preparations.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @note Must be called outside of a latent action. Preferably within BEFORE_TEST.
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.MapTest` CVar.
	 */
	CQTEST_API void AddWaitUntilLoadedCommand(FAutomationTestBase* TestRunner, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Finds the first pawn in the given map.
	 */
	CQTEST_API APawn* FindFirstPlayerPawn();

protected:
	/** Returns a newly created world. */
    CQTEST_API virtual UWorld* CreateWorld() override;

private:
	/** Handler called when the PIE session ends. */
	CQTEST_API void OnEndPlayMap();

	FString MapDirectory;
	FString MapName;
	UWorld* PieWorld{ nullptr };
	FDelegateHandle EndPlayMapHandle;
};

#endif // WITH_AUTOMATION_TESTS
