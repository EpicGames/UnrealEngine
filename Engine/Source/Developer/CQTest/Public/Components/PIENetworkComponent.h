// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

#define ENABLE_PIE_NETWORK_TEST WITH_EDITOR && (WITH_DEV_AUTOMATION_TESTS || WITH_PERF_AUTOMATION_TESTS)

#if ENABLE_PIE_NETWORK_TEST

#include "CQTest.h"
#include "CQTestSettings.h"
#include "PIENetworkTestStateRestorer.h"

#include "Editor.h"
#include "Engine/NetDriver.h"
#include "Engine/PackageMapClient.h"
#include "Net/NetIDVariant.h"
#include "UnrealEdGlobals.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

/*
//Example boiler plate

#include "CQTest.h"
#include "Components/PIENetworkComponent.h"

#include "GameFramework/GameModeBase.h"

#if ENABLE_PIE_NETWORK_TEST

NETWORK_TEST_CLASS(MyFixtureName, "Network.Example")
{
	// Deriving from the FBasePIENetworkComponentState to add objects to be tested in each state
	struct DerivedState : public FBasePIENetworkComponentState
	{
		APawn* ReplicatedPawn;
	};

	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithClients(2)
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}

	TEST_METHOD(SpawnAndReplicatePawn_WithReplicatedPawn_ProvidesPawnToClients)
	{
		Network.SpawnAndReplicate<APawn, &DerivedState::ReplicatedPawn>()
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(IsNotNull(ServerState.ReplicatedPawn));
			})
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedPawn));
			});
	}
};

#endif // ENABLE_PIE_NETWORK_TEST
*/

/**
 * Struct which handles the PIE session's world and network information.
 * Both the server and client will keep a reference to their own respective state which will be tested against.
 */
struct FBasePIENetworkComponentState
{
	/** Default virtual destructor. */
	virtual ~FBasePIENetworkComponentState() = default;

	/** Reference to the session's world. */
	UWorld* World = nullptr;

	/** Used by the server to reference the connections to the client. */
	TArray<UNetConnection*> ClientConnections;

	/** Used by the client to reference location within the `ClientConnections` array on the server. */
	int32 ClientIndex = INDEX_NONE;

	/** Used by the server for creating and validation of client instances. */
	int32 ClientCount = 2;

	/** Used by the server to create a PIE session as a dedicated or listen server. */
	bool bIsDedicatedServer = true;

	/** Used to track spawned and replicated actors across client and server PIE sessions. */
	TSet<UE::Net::FNetIDVariant> LocallySpawnedActors{};
};

/**
 * Component which creates and initializes the server and client network connections between the PIE sessions.
 * The component also provides methods around the CommandBuilder for adding latent commands.
 */
class FBasePIENetworkComponent
{
public:
	/**
	 * Construct the BasePIENetworkComponent.
	 *
	 * @param InTestRunner - Pointer to the TestRunner used for test reporting.
	 * @param InCommandBuilder - Reference to the latent command manager.
	 * @param IsInitializing - Bool specifying if the Automation framework is still being initialized.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @note Requires that the `FBasePIENetworkComponent` is built using the `FNetworkComponentBuilder` as this will setup the server and client `FBasePIENetworkComponentState`
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FBasePIENetworkComponent(FAutomationTestBase* InTestRunner, FTestCommandBuilder& InCommandBuilder, bool IsInitializing, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame.
	 * 
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	CQTEST_API FBasePIENetworkComponent& Then(TFunction<void()> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	CQTEST_API FBasePIENetworkComponent& Then(const TCHAR* Description, TFunction<void()> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame.
	 *
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	CQTEST_API FBasePIENetworkComponent& Do(TFunction<void()> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	CQTEST_API FBasePIENetworkComponent& Do(const TCHAR* Description, TFunction<void()> Action);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout.
	 *
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note Will continue to execute until the function provided evaluates to `true` or the duration exceeds the specified timeout.
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FBasePIENetworkComponent& Until(TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 *
	 * @note Will continue to execute until the function provided evaluates to `true` or the duration exceeds the specified timeout.
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FBasePIENetworkComponent& Until(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);
	
	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout.
	 *
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 *
	 * @note Will continue to execute until the function provided evaluates to `true` or the duration exceeds the specified timeout.
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FBasePIENetworkComponent& StartWhen(TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 *
	 * @note Will continue to execute until the function provided evaluates to `true` or the duration exceeds the specified timeout.
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FBasePIENetworkComponent& StartWhen(const TCHAR* Description, TFunction<bool()> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

protected:
	/** Prepares the active PIE sessions to be stopped. */
	CQTEST_API void StopPie();

	/** Setup settings for a network session and start PIE sessions for both the server and client with the network settings applied. */
	CQTEST_API void StartPie();
	
	/**
	 * Fetch all of the worlds that will be used for the networked session.
	 *
	 * @return true if an error was encountered or if the PIE sessions were created with the correct network settings, false otherwise.
	 *
	 * @note Method is expected to be used within the `Until` latent command to then wait until the worlds are ready for use.
	 */
	CQTEST_API bool SetWorlds();

	/** Sets the packet settings used to simulate packet behavior over a network. */
	CQTEST_API void SetPacketSettings() const;

	/** Connects all available clients to the server. */
	CQTEST_API void ConnectClientsToServer();

	/**
	 * Go through all of the client connections to make sure they are connected and ready.
	 *
	 * @return true if an error was encountered or if the connections all have a valid controller, false otherwise.
	 *
	 * @note Method is expected to be used within the `Until` latent command to then wait until the worlds are ready for use.
	 */
	CQTEST_API bool AwaitClientsReady() const;

	/** Restore back to the state prior to test execution. */
	CQTEST_API void RestoreState();

	/**
	 * Creates a timeout to be used.
	 * 
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return either the FTimespan specified or the value from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	CQTEST_API FTimespan MakeTimeout(TOptional<FTimespan> Timeout);

	TUniquePtr<FBasePIENetworkComponentState> ServerState = nullptr;
	TArray<TUniquePtr<FBasePIENetworkComponentState>> ClientStates;
	FAutomationTestBase* TestRunner = nullptr;
	FTestCommandBuilder* CommandBuilder = nullptr;
	FPacketSimulationSettings* PacketSimulationSettings = nullptr;
	TSubclassOf<AGameModeBase> GameMode = TSubclassOf<AGameModeBase>(nullptr);
	FPIENetworkTestStateRestorer StateRestorer;
	TMap<UE::Net::FNetIDVariant, int64> SpawnedActors{};
};

template <typename NetworkDataType>
class FNetworkComponentBuilder;

/**
 * Component which creates and initializes the server and client network connections between the PIE sessions.
 * Expands on the `FBasePIENetworkComponent` by providing separate methods to add latent commands for the server and clients.
 */
template <typename NetworkDataType = FBasePIENetworkComponentState>
class FPIENetworkComponent : public FBasePIENetworkComponent
{
public:
	/**
	 * Construct the PIENetworkComponent.
	 *
	 * @param InTestRunner - Pointer to the TestRunner used for test reporting.
	 * @param InCommandBuilder - Reference to the latent command manager.
	 * @param IsInitializing - Bool specifying if the Automation framework is still being initialized.
	 *
	 * @note Requires that the `FPIENetworkComponent` is built using the `FNetworkComponentBuilder` as this will setup the server and client `NetworkDataType` states
	 */
	FPIENetworkComponent(FAutomationTestBase* InTestRunner, FTestCommandBuilder& InCommandBuilder, bool IsInitializing)
		: FBasePIENetworkComponent(InTestRunner, InCommandBuilder, IsInitializing) {}

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on the server.
	 *
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenServer(TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on the server.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenServer(const TCHAR* Description, TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on all clients.
	 *
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenClients(TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on all clients.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenClients(const TCHAR* Description, TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on a given client.
	 *
	 * @param ClientIndex - Index of the connected client.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenClient(int32 ClientIndex, TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution on a subsequent frame on a given client.
	 *
	 * @param Description - Description of the provided function to be logged.
	 * @param ClientIndex - Index of the connected client.
	 * @param Action - Function to be executed.
	 * 
	 * @return a reference to this
	 */
	FPIENetworkComponent& ThenClient(const TCHAR* Description, int32 ClientIndex, TFunction<void(NetworkDataType&)> Action);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on the server.
	 *
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilServer(TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on the server.
	 *
	 * @param Description - Description of the latent command
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilServer(const TCHAR* Description, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on all clients.
	 *
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilClients(TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on all clients.
	 *
	 * @param Description - Description of the latent command
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilClients(const TCHAR* Description, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);
	
	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on a given client.
	 *
	 * @param ClientIndex - Index of the connected client.
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilClient(int32 ClientIndex, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Enqueues a function to the latent command manager for execution until completion or timeout on a given client.
	 *
	 * @param Description - Description of the latent command
	 * @param ClientIndex - Index of the connected client.
	 * @param Query - Function to be executed.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 * 
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& UntilClient(const TCHAR* Description, int32 ClientIndex, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Has a new client request a late join and syncs the server and other clients with the new client PIE session.
	 * 
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	FPIENetworkComponent& ThenClientJoins(TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);
	
	/**
	 * Spawns an actor on the server and replicates to the clients.
	 * 
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 * 
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::*ResultStorage>
	FPIENetworkComponent& SpawnAndReplicate(TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Spawns an actor on the server and replicates to the clients.
	 *
	 * @param SpawnParameters - Parameters that can be used when spawning an Actor.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 *
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::*ResultStorage>
	FPIENetworkComponent& SpawnAndReplicate(const FActorSpawnParameters& SpawnParameters, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Spawns an actor on the server and replicates to the clients.
	 *
	 * @param BeforeReplicate - Function to be called after Actor has been spawned, but before being replicated to the other clients.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 *
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
	FPIENetworkComponent& SpawnAndReplicate(TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Spawns an actor on the server and replicates to the connected clients.
	 *
	 * @param SpawnParameters - Parameters that can be used when spawning an Actor.
	 * @param BeforeReplicate - Function to be called after Actor has been spawned, but before being replicated to the other clients.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 *
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
	FPIENetworkComponent& SpawnAndReplicate(const FActorSpawnParameters& SpawnParameters, TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

private:
	friend class FNetworkComponentBuilder<NetworkDataType>;

	/**
	 * Spawns an actor on the server.
	 *
	 * @param SpawnParameters - Parameters that can be used when spawning an Actor.
	 * @param BeforeReplicate - Function to be called after Actor has been spawned.
	 * @param Timeout - Duration that the latent command can execute before reporting an error.
	 *
	 * @return a reference to this
	 *
	 * @note If no value is provided for the timeout, value will be pulled from the `TestFramework.CQTest.CommandTimeout.Network` CVar.
	 */
	template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
	FPIENetworkComponent& SpawnOnServer(const FActorSpawnParameters& SpawnParameters, TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout = CQTest::DefaultTimeout);

	/**
	 * Replicates all Actors to the connected client.
	 *
	 * @param ClientState - Client state to receive the updated Actor.
	 *
	 * @return true if all Actors have been successfully replicated on this client
	 */
	bool ReplicateToClients(NetworkDataType& ClientState);
};


/**
 * Helper object used to setup and build the `FPIENetworkComponent`.
 */
template <typename NetworkDataType = FBasePIENetworkComponentState>
class FNetworkComponentBuilder
{
public:
	/**
	 * Construct the NetworkComponentBuilder.
	 *
	 * @note Will assert if the `NetworkDataType` used is not of type or derived from `FBasePIENetworkComponentState`
	 */
	FNetworkComponentBuilder();

	/**
	 * Specifies the number of clients to be used.
	 *
	 * @param ClientCount - Number of clients not including the listen server.
	 * 
	 * @return a reference to this
	 * 
	 * @note Number of clients should exclude the server, even if using the server as a listen server.
	 */
	FNetworkComponentBuilder& WithClients(int32 ClientCount);

	/**
	 * Build the server as a dedicated server.
	 * 
	 * @return a reference to this
	 */
	FNetworkComponentBuilder& AsDedicatedServer();

	/**
	 * Build the server as a listen server.
	 *
	 * @return a reference to this
	 */
	FNetworkComponentBuilder& AsListenServer();

	/**
	 * Build the FPIENetworkComponent to use the provided packet simulation settings.
	 *
	 * @param InPacketSimulationSettings - Settings used to simulate packet behavior over a network.
	 * 
	 * @return a reference to this
	 */
	FNetworkComponentBuilder& WithPacketSimulationSettings(FPacketSimulationSettings* InPacketSimulationSettings);

	/**
	 * Build the FPIENetworkComponent to use the provided game mode.
	 *
	 * @param InGameMode - GameMode used for the networked session.
	 * 
	 * @return a reference to this
	 */
	FNetworkComponentBuilder& WithGameMode(TSubclassOf<AGameModeBase> InGameMode);

	/**
	 * Build the FPIENetworkComponent to restore back to the provided game instance.
	 *
	 * @param InGameInstanceClass - GameInstance to restore back to.
	 * 
	 * @return a reference to this
	 */
	FNetworkComponentBuilder& WithGameInstanceClass(FSoftClassPath InGameInstanceClass);

	/**
	 * Build the FPIENetworkComponent with the provided data.
	 *
	 * @param OutNetwork - Reference to the FPIENetworkComponent.
	 */
	void Build(FPIENetworkComponent<NetworkDataType>& OutNetwork);

private:
	FPacketSimulationSettings* PacketSimulationSettings = nullptr;
	TSubclassOf<AGameModeBase> GameMode = TSubclassOf<AGameModeBase>(nullptr);
	FSoftClassPath GameInstanceClass = FSoftClassPath{};
	int32 ClientCount = 2;
	bool bIsDedicatedServer = true;
};

constexpr static EAutomationTestFlags NetworkTestContext = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter;

/** Macro to quickly create tests which will only run within the Editor. */
#define NETWORK_TEST_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_FLAGS(_ClassName, _TestDir, NetworkTestContext)

#include "Impl/PIENetworkComponent.inl"

#endif // ENABLE_PIE_NETWORK_TEST
