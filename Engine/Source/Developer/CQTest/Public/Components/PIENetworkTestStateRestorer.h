// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/SoftObjectPath.h"
#include "Templates/SubclassOf.h"

#define UE_API CQTEST_API

class UWorld;
class AGameModeBase;

/**
 * Object which saves the state of the game before starting the test,
 * Then restores it after the test is complete.
 */
class FPIENetworkTestStateRestorer
{
public:
	FPIENetworkTestStateRestorer() = default;

	/**
	 * Construct the PIENetworkTestStateRestorer.
	 *
	 * @param InGameInstanceClass - Current game instance prior to the test.
	 * @param InGameMode - Current game mode prior to the test.
	 */
	UE_API FPIENetworkTestStateRestorer(const FSoftClassPath InGameInstanceClass, TSubclassOf<AGameModeBase> InGameMode);

	/** Restore to the previous game instance and mode. */
	UE_API void Restore();

private:
	bool SetWasLoadedFlag = false;
	FSoftClassPath OriginalGameInstance = FSoftClassPath();
	TSubclassOf<AGameModeBase> OriginalGameMode;
};

#undef UE_API
