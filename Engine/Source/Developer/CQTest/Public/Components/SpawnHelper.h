// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "UObject/GCObject.h"

#define UE_API CQTEST_API

/**
 * Helper object for spawning Actors and other object types in the world.
 * 
 * @see FActorTestSpawner, FMapTestSpawner
 */
struct FSpawnHelper
{
	/** Destruct the Spawn Helper. */
	UE_API virtual ~FSpawnHelper();

	/**
	 * Spawn an Actor in the world.
	 *
	 * @param SpawnParameters - Struct of optional parameters used to assist with spawning.
	 * @param Class - Class of the object to be spawned.
	 * 
	 * @return reference to the spawned object
	 * 
	 * @note Method guarantees that the Actor returned is valid, will assert otherwise.
	 */
	template <typename ActorType>
	ActorType& SpawnActor(const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters(), UClass* Class = nullptr)
	{
		return SpawnActorAtInWorld<ActorType>(GetWorld(), FVector::ZeroVector, FRotator::ZeroRotator, SpawnParameters, Class);
	}

	/**
	 * Spawn an Actor in the world.
	 *
	 * @param Location - Location to spawn the Actor in the world.
	 * @param Rotation - Rotation to spawn the Actor in the world.
	 * @param SpawnParameters - Struct of optional parameters used to assist with spawning.
	 * @param Class - Class of the object to be spawned.
	 * 
	 * @return reference to the spawned object
	 *
	 * @note Method guarantees that the Actor returned is valid, will assert otherwise.
	 */
	template <typename ActorType>
	ActorType& SpawnActorAt(FVector const& Location, FRotator const& Rotation, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters(), UClass* Class = nullptr)
	{
		return SpawnActorAtInWorld<ActorType>(GetWorld(), Location, Rotation, SpawnParameters, Class);
	}

	/**
	 * Create a new Object.
	 *
	 * @return reference to the created object
	 */
	template <typename ObjectType>
	ObjectType& SpawnObject()
	{
		static_assert(!TIsDerivedFrom<ObjectType, AActor>::IsDerived, "Use SpawnActor to spawn AActors.");
		static_assert(TIsDerivedFrom<ObjectType, UObject>::IsDerived, "Objects must derive from UObject.");
		ObjectType* const Object = NewObject<ObjectType>();
		check(Object != nullptr);
		SpawnedObjects.Add(Object);
		return *Object;
	}

	/** Returns a reference to the current world. */
	UE_API UWorld& GetWorld();

protected:
	/**
	 * Creates a new world.
	 * 
	 * @returns a newly created world.
	 */
	virtual UWorld* CreateWorld() = 0;

	TArray<TWeakObjectPtr<AActor>> SpawnedActors{};
	TArray<TWeakObjectPtr<UObject>> SpawnedObjects{};

	UWorld* GameWorld{ nullptr };

private:
	/**
	 * Spawn an Actor in the world.
	 *
	 * @param World - World to spawn the Actor.
	 * @param Location - Location to spawn the Actor in the world.
	 * @param Rotation - Rotation to spawn the Actor in the world.
	 * @param SpawnParameters - Struct of optional parameters used to assist with spawning.
	 * @param Class - Class of the object to be spawned.
	 * 
	 * @return reference to the spawned object
	 *
	 * @note Method guarantees that the Actor returned is valid, will assert otherwise.
	 */
	template <typename ActorType>
	ActorType& SpawnActorAtInWorld(UWorld& World, const FVector& Location, const FRotator& Rotation, const FActorSpawnParameters& SpawnParameters = FActorSpawnParameters(), UClass* Class = nullptr)
	{
		static_assert(TIsDerivedFrom<ActorType, AActor>::IsDerived, "Provided type does not derive from AActor");

		ActorType* const Actor = Class ? World.SpawnActor<ActorType>(Class, Location, Rotation, SpawnParameters) : World.SpawnActor<ActorType>(Location, Rotation, SpawnParameters);

		check(Actor != nullptr);
		SpawnedActors.Add(Actor);
		return *Actor;
	}
};

#undef UE_API
