// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"

#define UE_API CQTEST_API

namespace CQTestAssetHelper 
{

/**
 * Helper object which generates a FARFilter to be used with the CQTestAssetHelper functions.
 * 
 * @see FARFilter
 */
class FAssetFilterBuilder
{
public:
	/** Construct the FAssetFilterBuilder. */
	FAssetFilterBuilder() = default;

	/**
	 * Specifies the package name to be matched against.
	 *
	 * @param PackageName - Full name of the package.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional names.
	 */
	UE_API FAssetFilterBuilder& WithPackageName(const FName& PackageName);

	/**
	 * Specifies the package path to be matched against.
	 *
	 * @param PackagePath - Path of the package.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional paths.
	 * @note The path will be sanitized prior to being added to the filer (e.g. `Game/Assets/` converts to `/Game/Assets`).
	 * @note The specified path, but not sub paths(by default), will be included. Sub paths will be included only if IncludeRecursivePaths is called with true.
	 */
	UE_API FAssetFilterBuilder& WithPackagePath(const FName& PackagePath);

	/**
	 * Specifies the paths of specific assets to match.
	 *
	 * @param SoftObjectPath - Top level asset path.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional paths.
	 */
	UE_API FAssetFilterBuilder& WithSoftObjectPath(const FSoftObjectPath& SoftObjectPath);

	/**
	 * Specifies the class paths of specific class instances to match.
	 *
	 * @param ClassPath - Name of the Class instance.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional paths.
	 * @note Instances of the specified classes, but not subclasses (by default), will be included. Derived classes will be included only if IncludeRecursiveClasses is called with true.
	 */
	UE_API FAssetFilterBuilder& WithClassPath(const FTopLevelAssetPath& ClassPath);

	/**
	 * Specifies the tag and value of an asset to match.
	 *
	 * @param Tag - Name of the asset property.
	 * @param Value - Value of the asset property.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional tag/value pairs.
	 * @note Will only work on assets with properties marked with the AssetRegistrySearchable flag.
	 */
	UE_API FAssetFilterBuilder& WithTagAndValue(const FName& Tag, TOptional<FString> Value);

	/**
	 * Specifies the class paths to exclude.
	 *
	 * @param RecursiveClassPathsExclusion - Name of the Class instance.
	 *
	 * @return a reference to this
	 *
	 * @note Method can be called multiple times to provide additional paths.
	 */
	UE_API FAssetFilterBuilder& WithRecursiveClassPathsExclusion(const FTopLevelAssetPath& RecursiveClassPathsExclusion);

	/**
	 * Enables the inclusion of sub paths.
	 *
	 * @return a reference to this
	 */
	UE_API FAssetFilterBuilder& IncludeRecursivePaths();

	/**
	 * Enables the inclusion of subclass paths.
	 *
	 * @return a reference to this
	 */
	UE_API FAssetFilterBuilder& IncludeRecursiveClasses();

	/**
	 * Enables matching only on assets found on disk and not in memory.
	 *
	 * @return a reference to this
	 */
	UE_API FAssetFilterBuilder& IncludeOnlyOnDiskAssets();

	/**
	 * Specifies matching only on assets with the specified flags.
	 *
	 * @param PackageFlags - Bitmask of package flags.
	 *
	 * @return a reference to this
	 */
	UE_API FAssetFilterBuilder& WithPackageFlags(uint32 PackageFlags);

	/**
	 * Specifies matching only on assets without the specified flags.
	 *
	 * @param PackageFlags - Bitmask of package flags.
	 *
	 * @return a reference to this
	 */
	UE_API FAssetFilterBuilder& WithoutPackageFlags(uint32 PackageFlags);

	/**
	 * Build the FARFilter with the provided data.
	 *
	 * @return the generated FARFilterparam OutFilter - Reference to the FARFilter.
	 */
	UE_API FARFilter Build();

private:
	FARFilter AssetRegistryFilter;
};

} // CQTestAssetHelper

#undef UE_API
