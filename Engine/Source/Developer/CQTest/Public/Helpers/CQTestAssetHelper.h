// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/Optional.h"
#include "UObject/Object.h"
#include "UObject/Class.h"

/*
//Example boiler plate

#if WITH_AUTOMATION_TESTS

#include "CQTest.h"
#include "Helpers/CQTestAssetHelper.h"

TEST_CLASS_WITH_FLAGS(MyFixtureName, "AssetHelper.Example", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)
{
	TEST_METHOD(Object_FromBlueprint_IsLoaded)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackagePath(TEXT("/Package/Path/To/Asset"))
			.Build();

		UClass* Class = CQTestAssetHelper::GetBlueprintClass(Filter, TEXT("B_Asset"));
		ASSERT_THAT(IsNotNull(Class));

		UObject* Object = CQTestAssetHelper::FindDataBlueprint(Filter, TEXT("B_Asset"));
		ASSERT_THAT(IsNotNull(Object));
	}
};

#endif // WITH_AUTOMATION_TESTS
*/

/**
 * Helper functions for working with assets in C++ tests
 * 
 * @note If your constructor invokes these methods, you should only run if bInitializing is false
 * to avoid missing assets from plugins.
 */
namespace CQTestAssetHelper 
{

/**
 * Looks for the package path of an asset by its name in the AssetRegistry.
 * 
 * @param Name - Name of the asset to find.
 * 
 * @return package path of the asset if found
 * 
 * @note If the asset name is found in multiple package paths, it's unspecified which one is returned.
 */
CQTEST_API TOptional<FString> FindAssetPackagePathByName(const FString& Name);

/**
 * Looks for the package path of an asset by its name in the AssetRegistry.
 * 
 * @param Filter - AssetRegistry filter.
 * @param Name - Name of the asset to find.
 * 
 * @return package path of the asset if found
 * 
 * @note If the asset name is found in multiple package paths, it's unspecified which one is returned.
 */
CQTEST_API TOptional<FString> FindAssetPackagePathByName(const FARFilter& Filter, const FString& Name);

/**
 * Looks for any assets matching the filter in the AssetRegistry.
 *
 * @param Filter - AssetRegistry filter.
 *
 * @return array of assets matching the filter
 */
CQTEST_API TArray<FAssetData> FindAssetsByFilter(const FARFilter& Filter);

/**
 * Looks for a Blueprint class by its name in the AssetRegistry.
 * 
 * @param Name - Name of the Blueprint to find.
 * 
 * @return UClass of the Blueprint if found
 * 
 * @note If the name is found in multiple package paths, it's unspecified which one is returned.
 */
CQTEST_API UClass* GetBlueprintClass(const FString& Name);

/**
 * Looks for a Blueprint class by its name in the AssetRegistry.
 * 
 * @param Filter - AssetRegistry filter.
 * @param Name - Name of the Blueprint to find.
 * 
 * @return UClass of the Blueprint if found
 * 
 * @note If the name is found in multiple package paths, it's unspecified which one is returned.
 */
CQTEST_API UClass* GetBlueprintClass(const FARFilter& Filter, const FString& Name);

/**
 * Looks for all Blueprint classes matching the filter in the AssetRegistry.
 *
 * @param Filter - AssetRegistry filter.
 *
 * @return array of available UClasses matching the filter
 */
CQTEST_API TArray<UClass*> GetBlueprintClasses(const FARFilter& Filter);

/**
  * Looks for a Data Blueprint by its name in the AssetRegistry.
  * 
  * @param Name - Name of the Data Blueprint to find.
  * 
  * @return UObject of the Data Blueprint if found
  * 
  * @note If the name is found in multiple package paths, it's unspecified which one is returned.
  */
CQTEST_API UObject* FindDataBlueprint(const FString& Name);

/**
 * Looks for a Data Blueprint by its name in the AssetRegistry.
 * 
 * @param Filter - AssetRegistry filter.
 * @param Name - Name of the Data Blueprint to find
 * 
 * @return UObject of the Data Blueprint if found
 * 
 * @note If the name is found in multiple package paths, it's unspecified which one is returned.
 */
CQTEST_API UObject* FindDataBlueprint(const FARFilter& Filter, const FString& Name);

/**
 * Looks for all UObjects matching the filter in the AssetRegistry.
 *
 * @param Filter - AssetRegistry filter.
 *
 * @return array of available UObjects matching the filter
 */
CQTEST_API TArray<UObject*> FindDataBlueprints(const FARFilter& Filter);

} // CQTestAssetHelper