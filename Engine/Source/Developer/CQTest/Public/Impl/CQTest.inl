// Copyright Epic Games, Inc. All Rights Reserved.

namespace
{
	DEFINE_LOG_CATEGORY_STATIC(LogCqTest, Log, All);

	template <typename AsserterType>
	struct TBeforeTestCommand : public IAutomationLatentCommand
	{
		TBeforeTestCommand(TBaseTest<AsserterType>& InCurrentTest, TSharedRef<FRunSequence> InSequence)
			: CurrentTest(InCurrentTest)
			, Sequence(InSequence)
		{}

		bool Update() override
		{
			UE_LOGF(LogCqTest, Log, "Before Test");

			if (GEngine != nullptr)
			{
				//Do not collect garbage during the test. We force GC at the end.
				GEngine->DelayGarbageCollection();
			}

			CurrentTest.Setup();
			if (auto LatentActions = CurrentTest.TestCommandBuilder.Build())
			{
				Sequence->Prepend(LatentActions);
			}

			return true;
		}

		TBaseTest<AsserterType>& CurrentTest;
		TSharedRef<FRunSequence> Sequence;
	};

	template <typename AsserterType>
	struct TRunTestCommand : public IAutomationLatentCommand
	{
		TRunTestCommand(TBaseTest<AsserterType>& InCurrentTest, const FString& InRequestedTest, const TTestRunner<AsserterType>& InTestRunner, TSharedRef<FRunSequence> InSequence)
			: CurrentTest(InCurrentTest)
			, RequestedTest(InRequestedTest)
			, TestRunner(InTestRunner)
			, Sequence(InSequence)
		{
		}

		bool Update() override
		{
			UE_LOGF(LogCqTest, Log, "RunTest");

			if (TestRunner.HasAnyErrors())
			{
				UE_LOGF(LogCqTest, Log, "Skipping Test due to existing errors");
				return true; // skip run if errors in BeforeTest
			}

			CurrentTest.RunTest(RequestedTest);
			if (auto LatentActions = CurrentTest.TestCommandBuilder.Build())
			{
				Sequence->Prepend(LatentActions);
			}

			return true;
		}

		TBaseTest<AsserterType>& CurrentTest;
		const FString& RequestedTest;
		const TTestRunner<AsserterType>& TestRunner;
		TSharedRef<FRunSequence> Sequence;
	};

	template <typename AsserterType>
	struct TAfterTestCommand : public IAutomationLatentCommand
	{
		TAfterTestCommand(TBaseTest<AsserterType>& InCurrentTest, TSharedRef<FRunSequence> InSequence)
			: CurrentTest(InCurrentTest)
			, Sequence(InSequence)
		{}

		bool Update() override
		{
			UE_LOGF(LogCqTest, Log, "Running After Test");

			CurrentTest.TearDown();
			if (auto LatentActions = CurrentTest.TestCommandBuilder.BuildTearDown())
			{
				Sequence->Prepend(LatentActions);
			}
			if (auto LatentActions = CurrentTest.TestCommandBuilder.Build())
			{
				Sequence->Prepend(LatentActions);
			}

			return true;
		}

		TBaseTest<AsserterType>& CurrentTest;
		TSharedRef<FRunSequence> Sequence;
	};

	template <typename AsserterType>
	struct TTearDownRunner : public IAutomationLatentCommand
	{
		explicit TTearDownRunner(TTestRunner<AsserterType>& TestRunner)
			: TestRunner(TestRunner) {}

		bool Update() override
		{
			UE_LOGF(LogCqTest, Log, "Tearing Down Test");

			TestRunner.SetSuppressLogWarnings(ECQTestSuppressLogBehavior::Default);
			TestRunner.SetSuppressLogErrors(ECQTestSuppressLogBehavior::Default);

			TestRunner.CurrentTestPtr = nullptr;
			if (GEngine != nullptr)
			{
				//Force GC at the end of every test.
				GEngine->ForceGarbageCollection();
			}

			return true;
		}

		TTestRunner<AsserterType>& TestRunner;
	};
} // namespace

template <typename AsserterType>
inline TTestRunner<AsserterType>::TTestRunner(FString InName, int32 InLineNumber, const char* InFileName, FString InTestDir, EAutomationTestFlags InTestFlags, TTestInstanceGenerator<AsserterType> InFactory, FString InTestTags)
	: FAutomationTestBase(InName, true)
	, LineNumber(InLineNumber)
	, FileName(FString(InFileName))
	, TestDir(InTestDir)
	, TestFlags(InTestFlags)
	, TestInstanceFactory(InFactory)
	, TestTags(InTestTags)
{
	bInitializing = true;
	if (TestDir.Equals(GenerateTestDirectory))
	{
		TestDir = TestDirectoryGenerator::Generate(FileName);
	}
	else if (TestDir.Contains(TEXT("[GenerateTestDirectory]"), ESearchCase::IgnoreCase))
	{
		TestDir = TestDir.Replace(TEXT("[GenerateTestDirectory]"), *TestDirectoryGenerator::Generate(FileName), ESearchCase::IgnoreCase);
	}

	CurrentTestPtr = TestInstanceFactory(*this);

	if (CurrentTestPtr->BeforeAllFunc)
	{
		BeforeAllDelegate = FAutomationTestFramework::Get().GetOnEnteringTestSection(GetBeautifiedTestName()).AddStatic(CurrentTestPtr->BeforeAllFunc);
	}
	if (CurrentTestPtr->AfterAllFunc)
	{
		AfterAllDelegate = FAutomationTestFramework::Get().GetOnLeavingTestSection(GetBeautifiedTestName()).AddStatic(CurrentTestPtr->AfterAllFunc);
	}

	bInitializing = false;
}

template <typename AsserterType>
inline TTestRunner<AsserterType>::~TTestRunner() {
	if (BeforeAllDelegate.IsValid())
	{
		FAutomationTestFramework::Get().GetOnEnteringTestSection(GetBeautifiedTestName()).Remove(BeforeAllDelegate);
	}
	if (AfterAllDelegate.IsValid())
	{
		FAutomationTestFramework::Get().GetOnLeavingTestSection(GetBeautifiedTestName()).Remove(AfterAllDelegate);
	}
}

template <typename AsserterType>
inline bool TTestRunner<AsserterType>::RunTest(const FString& RequestedTest)
{
	if (RequestedTest.Len() == 0)
	{
		return false;
	}

	CurrentTestPtr = TestInstanceFactory(*this);
	check(CurrentTestPtr != nullptr);
	auto& CurrentTest = *CurrentTestPtr;

	TSharedRef<FRunSequence> CommandSequence = MakeShareable<FRunSequence>(new FRunSequence());
	auto Before = MakeShared<TBeforeTestCommand<AsserterType>>(CurrentTest, CommandSequence);
	auto Run = MakeShared<TRunTestCommand<AsserterType>>(CurrentTest, RequestedTest, *this, CommandSequence);
	auto After = MakeShared<TAfterTestCommand<AsserterType>>(CurrentTest, CommandSequence);
	auto TearDown = MakeShared<TTearDownRunner<AsserterType>>(*this);

	auto RemainingSteps = TArray<TSharedPtr<IAutomationLatentCommand>>{ Before, Run, After, TearDown };

	while (!RemainingSteps.IsEmpty())
	{
		RemainingSteps[0]->Update();
		RemainingSteps.RemoveAt(0);
		if (CurrentTestPtr != nullptr && !CommandSequence->IsEmpty())
		{
			CommandSequence->AppendAll(RemainingSteps);
			FAutomationTestFramework::Get().EnqueueLatentCommand(CommandSequence);
			return true;
		}
	}

	return true;
}

template <typename AsserterType>
inline FString TTestRunner<AsserterType>::GetBeautifiedTestName() const
{
	return FString::Printf(TEXT("%s.%s"), *TestDir, *TestName);
}

template <typename AsserterType>
inline uint32 TTestRunner<AsserterType>::GetRequiredDeviceNum() const
{
	return 1;
}

template <typename AsserterType>
inline EAutomationTestFlags TTestRunner<AsserterType>::GetTestFlags() const
{
	return TestFlags;
}

template <typename AsserterType>
inline FString TTestRunner<AsserterType>::GetTestSourceFileName() const
{
	return FileName;
}

template <typename AsserterType>
inline int32 TTestRunner<AsserterType>::GetTestSourceFileLine() const
{
	return LineNumber;
}

template <typename AsserterType>
inline int32 TTestRunner<AsserterType>::GetTestSourceFileLine(const FString& Name) const
{
	FString TestParam(Name);
	int32 Pos = Name.Find(TEXT(" "));
	if (Pos != INDEX_NONE)
	{
		TestParam = Name.RightChop(Pos + 1);
	}
	if (TestNames.Contains(TestParam))
	{
		return TestLineNumbers[TestParam];
	}
	return GetTestSourceFileLine();
}

template <typename AsserterType>
inline bool TTestRunner<AsserterType>::SuppressLogWarnings()
{
	if (SuppressLogWarningsBehavior == ECQTestSuppressLogBehavior::Default)
	{
		return bSuppressLogWarnings;
	}
	return SuppressLogWarningsBehavior == ECQTestSuppressLogBehavior::True;
}

template <typename AsserterType>
inline bool TTestRunner<AsserterType>::SuppressLogErrors()
{
	if (SuppressLogErrorsBehavior == ECQTestSuppressLogBehavior::Default)
	{
		return bSuppressLogErrors;
	}
	return SuppressLogErrorsBehavior == ECQTestSuppressLogBehavior::True;
}

template <typename AsserterType>
inline void TTestRunner<AsserterType>::SetSuppressLogWarnings(ECQTestSuppressLogBehavior Behavior)
{
	SuppressLogWarningsBehavior = Behavior;
}

template <typename AsserterType>
inline void TTestRunner<AsserterType>::SetSuppressLogErrors(ECQTestSuppressLogBehavior Behavior)
{
	SuppressLogErrorsBehavior = Behavior;
}

template <typename AsserterType>
inline void TTestRunner<AsserterType>::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	for (const auto& testName : TestNames)
	{
		OutBeautifiedNames.Add(testName);
		OutTestCommands.Add(testName);
	}
}

/////////////

template <typename Derived, typename AsserterType>
inline void TTest<Derived, AsserterType>::RunTest(const FString& TestName)
{
	auto TestMethod = Methods[TestName];
	Derived& Self = static_cast<Derived&>(*this);

	(Self.*(TestMethod))();
}

template <typename AsserterType>
inline TBaseTest<AsserterType>::TBaseTest(FAutomationTestBase& TestRunner, bool bInitializing)
	: bInitializing(bInitializing)
	, TestRunner(TestRunner)
	, Assert(AsserterType{ TestRunner })
	, TestCommandBuilder(FTestCommandBuilder{ TestRunner })
{
}

template <typename AsserterType>
inline void TBaseTest<AsserterType>::AddCommand(IAutomationLatentCommand* Cmd)
{
	TestCommandBuilder.CommandQueue.Add(MakeShareable<IAutomationLatentCommand>(Cmd));
}

template <typename AsserterType>
inline void TBaseTest<AsserterType>::AddCommand(TSharedPtr<IAutomationLatentCommand> Cmd)
{
	TestCommandBuilder.CommandQueue.Add(Cmd);
}

template <typename AsserterType>
inline void TBaseTest<AsserterType>::AddError(const FString& InError) const
{
	TestRunner.AddError(InError, 0);
}

template <typename AsserterType>
inline bool TBaseTest<AsserterType>::AddErrorIfFalse(bool bCondition, const FString& InError) const
{
	return TestRunner.AddErrorIfFalse(bCondition, InError, 0);
}

template <typename AsserterType>
inline void TBaseTest<AsserterType>::AddWarning(const FString& InWarning) const
{
	TestRunner.AddWarning(InWarning, 0);
}

template <typename AsserterType>
inline void TBaseTest<AsserterType>::AddInfo(const FString& InLogItem) const
{
	TestRunner.AddInfo(InLogItem, 0, false);
}