// Copyright Epic Games, Inc. All Rights Reserved.

///////////////////////////////////////////////////////////////////////
// FPIENetworkComponent

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenServer(TFunction<void(NetworkDataType&)> Action)
{
	return ThenServer(nullptr, Action);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenServer(const TCHAR* Description, TFunction<void(NetworkDataType&)> Action)
{
	CommandBuilder->Do(Description, [this, Action] { Action(static_cast<NetworkDataType&>(*ServerState)); });
	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenClients(TFunction<void(NetworkDataType&)> Action)
{
	return ThenClients(nullptr, Action);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenClients(const TCHAR* Description, TFunction<void(NetworkDataType&)> Action)
{
	// The outer Do is to delay the for-loop until execution time in case a client joins during the test
	CommandBuilder->Do(Description, [this, Action]() { 
		for (int32 Index = 0; Index < ClientStates.Num(); Index++)
		{
			Action(static_cast<NetworkDataType&>(*ClientStates[Index]));
		}
	});

	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenClient(int32 ClientIndex, TFunction<void(NetworkDataType&)> Action)
{
	return ThenClient(nullptr, ClientIndex, Action);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenClient(const TCHAR* Description, int32 ClientIndex, TFunction<void(NetworkDataType&)> Action)
{
	// The outer Do is to delay the for-loop until execution time in case a client joins during the test
	CommandBuilder->Do(Description, [this, Action, ClientIndex]() { 
		if (!ClientStates.IsValidIndex(ClientIndex))
		{
			TestRunner->AddError(FString::Printf(TEXT("Invalid client index specified. Requested Index: %d MaxIndex: %d"), ClientIndex, ClientStates.Num() - 1));
			return;
		}
		Action(static_cast<NetworkDataType&>(*ClientStates[ClientIndex]));
	});
		
	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilServer(TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	return UntilServer(nullptr, Query, Timeout);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilServer(const TCHAR* Description, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->Until(Description, [this, Query]() { 
		return Query(static_cast<NetworkDataType&>(*ServerState)); 
	}, TimeoutValue);

	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilClients(TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	return UntilClients(nullptr, Query, Timeout);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilClients(const TCHAR* Description, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	// Capture a mutable array of bools (by value) to track which clients have already finished to avoid calling them again
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	TArray<bool> ClientsFinishedTask{};
	CommandBuilder->Until(Description, [this, Query, ClientsFinishedTask]() mutable {
		// Array with be initially empty and will need to be resized to match our current client count
		// It is safe to assume that no clients will join in the middle of this action so the resize will occur only once
		if (ClientsFinishedTask.Num() < ClientStates.Num())
		{
			ClientsFinishedTask.SetNumZeroed(ClientStates.Num());
		}

		bool bIsAllDone = true;
		for (int32 Index = 0; Index < ClientStates.Num(); Index++) 
		{
			if (!ClientsFinishedTask[Index])
			{
				if (Query(static_cast<NetworkDataType&>(*ClientStates[Index]))) 
				{
					ClientsFinishedTask[Index] = true;
				}
				else 
				{
					bIsAllDone = false;
				}
			}
		}

		return bIsAllDone;
	}, TimeoutValue);

	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilClient(int32 ClientIndex, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	return UntilClient(nullptr, ClientIndex, Query, Timeout);
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::UntilClient(const TCHAR* Description, int32 ClientIndex, TFunction<bool(NetworkDataType&)> Query, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	CommandBuilder->Until(Description, [this, Query, ClientIndex]() {
		if (!ClientStates.IsValidIndex(ClientIndex))
		{
			TestRunner->AddError(FString::Printf(TEXT("Invalid client index specified. Requested Index: %d MaxIndex: %d"), ClientIndex, ClientStates.Num() - 1));
			return true;
		}
		return Query(static_cast<NetworkDataType&>(*ClientStates[ClientIndex]));
	}, TimeoutValue);

	return *this;
}

template <typename NetworkDataType>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::ThenClientJoins(TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	Do(TEXT("Update Server State"), [this]() {
		int32 NextClientIndex = ServerState->ClientCount++;
		ClientStates.Add(MakeUnique<NetworkDataType>(NetworkDataType{}));
		ClientStates.Last()->ClientIndex = NextClientIndex;
		ServerState->ClientConnections.SetNum(ServerState->ClientCount);
		GEditor->RequestLateJoin();
	})
	.Until(TEXT("Setting Worlds"), [this]() { return SetWorlds(); }, TimeoutValue)
	.Then(TEXT("Setup Packet Settings"), [this]() { SetPacketSettings(); })
	.Then(TEXT("Connect Clients to Server"), [this]() { ConnectClientsToServer(); });

	UntilClient(TEXT("Replicate to new Client"), ServerState->ClientCount, [this](NetworkDataType& State) {
		return ReplicateToClients(State);
	}, TimeoutValue);
	
	return *this;
}

template <typename NetworkDataType>
template <typename ActorToSpawn, ActorToSpawn* NetworkDataType::*ResultStorage>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::SpawnAndReplicate(TOptional<FTimespan> Timeout)
{
	return SpawnAndReplicate<ActorToSpawn, ResultStorage>({}, {}, Timeout);
}

template <typename NetworkDataType>
template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::SpawnAndReplicate(const FActorSpawnParameters& SpawnParameters, TOptional<FTimespan> Timeout) {
	return SpawnAndReplicate<ActorToSpawn, ResultStorage>(SpawnParameters, {}, Timeout);
}

template <typename NetworkDataType>
template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::SpawnAndReplicate(TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout) {
	return SpawnAndReplicate<ActorToSpawn, ResultStorage>({}, BeforeReplicate, Timeout);
}

template <typename NetworkDataType>
template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::SpawnAndReplicate(const FActorSpawnParameters& SpawnParameters, TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout)
{
	FTimespan TimeoutValue = MakeTimeout(Timeout);
	SpawnOnServer<ActorToSpawn, ResultStorage>(SpawnParameters, BeforeReplicate, TimeoutValue);
	UntilClients([this](NetworkDataType& ClientState) { return ReplicateToClients(ClientState); }, TimeoutValue);
		
	return *this;
}

template <typename NetworkDataType>
template<typename ActorToSpawn, ActorToSpawn* NetworkDataType::* ResultStorage>
inline FPIENetworkComponent<NetworkDataType>& FPIENetworkComponent<NetworkDataType>::SpawnOnServer(const FActorSpawnParameters& SpawnParameters, TFunction<void(ActorToSpawn&)> BeforeReplicate, TOptional<FTimespan> Timeout) {
	static_assert(std::is_convertible_v<ActorToSpawn*, AActor*>, "ActorToSpawn must derive from AActor");
	static_assert(std::is_default_constructible<NetworkDataType>::value, "NetworkDataType must have a default constructor accessible");

	FTimespan TimeoutValue = MakeTimeout(Timeout);
	// TODO: Consider passing in a constructed actor from an FTestSpawner instead
	// That would allow using TObjectBuilder as well.
	// Need a version which takes the Server's world instead of creating its own

	TSharedPtr<ActorToSpawn*> ServerActor = MakeShareable(new ActorToSpawn * (nullptr));

	ThenServer(TEXT("Spawning Actor On Server"), [ServerActor, SpawnParameters, BeforeReplicate](NetworkDataType& State) {
		*ServerActor = State.World->template SpawnActor<ActorToSpawn>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnParameters);
		if(BeforeReplicate)
		{
			BeforeReplicate(**ServerActor);
		}
		if (ResultStorage != nullptr)
		{
			State.*ResultStorage = *ServerActor;
		}
	})
	.UntilServer(TEXT("Waiting for Net ID"), [this, ServerActor](NetworkDataType& State) {
		UE::Net::FNetIDVariant NetIDVariant;
		if (State.World->GetNetDriver()->IsUsingIrisReplication())
		{
			UReplicationSystem* ReplicationSystem = State.World->GetNetDriver()->GetReplicationSystem();
			UObjectReplicationBridge* ReplicationBridge = Cast<UObjectReplicationBridge>(ReplicationSystem->GetReplicationBridge());
			checkf(IsValid(ReplicationBridge), TEXT("Unable to create a ReplicationBridge."));
			NetIDVariant = UE::Net::FNetIDVariant(ReplicationBridge->GetReplicatedRefHandle(*ServerActor));
		}
		else
		{
			NetIDVariant = UE::Net::FNetIDVariant(State.World->GetNetDriver()->GetNetGuidCache()->GetNetGUID(*ServerActor));
		}

		if (!NetIDVariant.IsValid())
		{
			return false;
		}

		// Calculate the pointer offset to the storage location on NetworkDataType
		// Do this by allocating a temporary NetworkDataType object and then calculate
		// the address of the storage location as an int64
		// This allows ReplicateToClients to not need the ActorToSpawn or ResultStorage template parameters
		// which in turn allows ThenClientJoins to use ReplicateToClients
		NetworkDataType TempDataType {};
		int64 StorageOffset = reinterpret_cast<int64>(&(TempDataType.*ResultStorage)) - reinterpret_cast<int64>(&TempDataType);

		SpawnedActors.Add(NetIDVariant, StorageOffset);
		State.LocallySpawnedActors.Add(NetIDVariant);
		return true;
	}, TimeoutValue);
	
	return *this;
}

template <typename NetworkDataType>
inline bool FPIENetworkComponent<NetworkDataType>::ReplicateToClients(NetworkDataType& ClientState)
{
	for(const auto& [NetIDVariant, StorageOffset] : SpawnedActors)
	{
		if(ClientState.LocallySpawnedActors.Contains(NetIDVariant))
		{
			continue;
		}

		AActor* ClientActor = nullptr;
		if (ClientState.World->GetNetDriver()->IsUsingIrisReplication())
		{
			UReplicationSystem* ReplicationSystem = ClientState.World->GetNetDriver()->GetReplicationSystem();
			UObjectReplicationBridge* ReplicationBridge = Cast<UObjectReplicationBridge>(ReplicationSystem->GetReplicationBridge());
			checkf(IsValid(ReplicationBridge), TEXT("Unable to create a ReplicationBridge."));
			ClientActor = Cast<AActor>(ReplicationBridge->GetReplicatedObject(NetIDVariant.GetVariant().template Get<UE::Net::FNetRefHandle>()));
		}
		else
		{
			ClientActor = Cast<AActor>(ClientState.World->GetNetDriver()->GetNetGuidCache()->GetObjectFromNetGUID(NetIDVariant.GetVariant().template Get<FNetworkGUID>(), true));
		}

		if (ClientActor != nullptr)
		{
			// The other half of the offset implementation.
			// Interpret the address of the State object as a 1-byte aligned pointer, and add the offset
			// Then interpret that as a pointer to a pointer to an AActor which we can assign
			AActor** Storage = reinterpret_cast<AActor**>(reinterpret_cast<char*>(&ClientState) + StorageOffset);
			*Storage = ClientActor;
			ClientState.LocallySpawnedActors.Add(NetIDVariant);
		}
	}

	return ClientState.LocallySpawnedActors.Num() == SpawnedActors.Num();
}

///////////////////////////////////////////////////////////////////////
// FNetworkComponentBuilder

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>::FNetworkComponentBuilder()
{
	static_assert(std::is_convertible_v<NetworkDataType*, FBasePIENetworkComponentState*>, "NetworkDataType must derive from FBaseNetworkComponentState");
}

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::WithClients(int32 InClientCount)
{
	checkf(InClientCount > 0, TEXT("Client count must be greater than 0.  Server only tests should simply use a Spawner"));
	ClientCount = InClientCount;
	return *this;
}

template<typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::WithPacketSimulationSettings(FPacketSimulationSettings* InPacketSimulationSettings)
{
	PacketSimulationSettings = InPacketSimulationSettings;
	return *this;
}

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::AsDedicatedServer()
{
	bIsDedicatedServer = true;
	return *this;
}

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::AsListenServer()
{
	bIsDedicatedServer = false;
	return *this;
}

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::WithGameMode(TSubclassOf<AGameModeBase> InGameMode)
{
	GameMode = InGameMode;
	return *this;
}

template <typename NetworkDataType>
inline FNetworkComponentBuilder<NetworkDataType>& FNetworkComponentBuilder<NetworkDataType>::WithGameInstanceClass(FSoftClassPath InGameInstanceClass)
{
	GameInstanceClass = InGameInstanceClass;
	return *this;
}

template<typename NetworkDataType>
inline void FNetworkComponentBuilder<NetworkDataType>::Build(FPIENetworkComponent<NetworkDataType>& OutNetwork)
{
	NetworkDataType DefaultState{};
	DefaultState.ClientCount = ClientCount;
	DefaultState.bIsDedicatedServer = bIsDedicatedServer;

	OutNetwork.ServerState = MakeUnique<NetworkDataType>(DefaultState);
	OutNetwork.ServerState->ClientConnections.SetNum(ClientCount);

	for (int32 ClientIndex = 0; ClientIndex < ClientCount; ClientIndex++)
	{
		OutNetwork.ClientStates.Add(MakeUnique<NetworkDataType>(DefaultState));
		OutNetwork.ClientStates.Last()->ClientIndex = ClientIndex;
	}

	OutNetwork.PacketSimulationSettings = PacketSimulationSettings;
	OutNetwork.GameMode = GameMode;
	OutNetwork.StateRestorer = FPIENetworkTestStateRestorer{GameInstanceClass, GameMode};
}
