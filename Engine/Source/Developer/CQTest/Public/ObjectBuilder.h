// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Misc/Build.h"

#if WITH_AUTOMATION_WORKER

#include "Containers/Array.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include "Components/SpawnHelper.h"

DEFINE_LOG_CATEGORY_STATIC(LogObjectBuilder, Log, Log);

template <typename TUObject>
class TObjectBuilder final
{
public:
	TObjectBuilder(UObject* InOwner = static_cast<UObject*>(GetTransientPackage()))
		: Object(NewObject<TUObject>(InOwner))
	{
		static_assert(std::is_base_of<UObject, TUObject>::value, "Template param must be a UObject type");
		static_assert(!std::is_base_of<AActor, TUObject>::value, "Template param must not be an Actor type, supply a FSpawnHelper to the constructor for Actor configuration");
		check(Object);
	}

	TObjectBuilder(FSpawnHelper& Spawner, UClass* Clazz = nullptr)
		: Object(nullptr)
	{
		static_assert(std::is_base_of<AActor, TUObject>::value, "Template param must be an Actor type, use the default constructor for non-actor UObjects");

		FActorSpawnParameters DeferredConstructionParams;
		DeferredConstructionParams.bDeferConstruction = true;

		Object = &Spawner.SpawnActor<TUObject>(DeferredConstructionParams, Clazz);
	}

	template <typename T>
	TObjectBuilder<TUObject>& SetParam(const FName& InPropertyName, T InValue)
	{
		using Raw = std::decay_t<T>;

		if (!Object)
		{
			UE_LOGF(LogObjectBuilder, Error, "Tried to SetParam on property %ls after Spawning. ObjectBuilder can only be used to paramaterise pre-spawn", *InPropertyName.ToString());
			return *this;
		}

		auto TrySet = [&](auto Property) {
			if (!Property)
			{
				UE_LOGF(LogObjectBuilder, Error, "Failed to find %ls property %ls on object %ls, check type and Property name are correct", *GetTypeName<Raw>(), *InPropertyName.ToString(), *Object->GetName());
				return;
			}
			if (!IsCompatibleWith<Raw>(*Property))
			{
				UE_LOGF(LogObjectBuilder, Error, "Type mismatch: Tried to set %ls property %ls on object %ls to a %ls", *Property->GetCPPType(nullptr, 0), *InPropertyName.ToString(), *Object->GetName(), *GetTypeName<Raw>());
				return;
			}

			if (auto TypeMatched = Property->template ContainerPtrToValuePtr<Raw>(Object))
			{
				*TypeMatched = InValue;
			}
			else
			{
				UE_LOGF(LogObjectBuilder, Error, "Container type mismatch: Could not cast from %ls to %ls", *Property->GetCPPType(nullptr, 0), *GetTypeName<Raw>());
			}
		};

		if constexpr (TIsArray<Raw>::Value)
		{
			TrySet(FindFProperty<FArrayProperty>(Object->GetClass(), InPropertyName));
		}
		else if constexpr (TIsTMap_V<Raw>)
		{
			TrySet(FindFProperty<FMapProperty>(Object->GetClass(), InPropertyName));
		}
		else if constexpr (TIsTSet_V<Raw>)
		{
			TrySet(FindFProperty<FSetProperty>(Object->GetClass(), InPropertyName));
		}
		else
		{
			if (auto Property = FindFProperty<FProperty>(Object->GetClass(), InPropertyName))
			{
				if (this->template IsCompatibleWith<Raw>(*Property))
				{
					Property->SetValue_InContainer(Object, &InValue);
				}
				else
				{
					UE_LOGF(LogObjectBuilder, Error, "Type mismatch: Tried to set %ls property %ls on object %ls to a %ls", *Property->GetCPPType(nullptr, 0), *InPropertyName.ToString(), *Object->GetName(), *this->template GetTypeName<Raw>());
				}
			}
			else
			{
				UE_LOGF(LogObjectBuilder, Error, "Failed to find %ls property %ls on object %ls, check type and Property name are correct", *GetTypeName<Raw>(), *InPropertyName.ToString(), *Object->GetName());
			}
		}

		return *this;
	}

	template<typename T = TUObject>
	T& Spawn(FTransform InTransform = FTransform::Identity) 
	{
		if (!Object)
		{
			UE_LOGF(LogObjectBuilder, Error, "Tried to spawn from builder multiple times.  Builder can only spawn a single object");
			auto* DefaultObject = NewObject<T>();
			return *DefaultObject;
		}

		if constexpr (std::is_base_of_v<AActor, T>)
		{
			Object->FinishSpawning(InTransform);
		}

		auto& ObjectRef = *Object;
		Object = nullptr;
		return ObjectRef;
	}

	template<typename TComponent, typename T = TUObject>
	TObjectBuilder<TUObject>& AddComponentTo(TComponent* InComponentToAdd = nullptr) 
	{
		static_assert(std::is_base_of_v<AActor, T>, "Can only add components to AActors");
		static_assert(std::is_base_of<UActorComponent, TComponent>::value, "Template param must be a UActorComponent type");
		if (!Object)
		{
			UE_LOGF(LogObjectBuilder, Error, "Tried to AddComponentTo actor after Spawning. ObjectBuilder can only be used to paramaterise pre-spawn");
			return *this;
		}

		TComponent* Component = InComponentToAdd;
		if (Component)
		{
			Component->Rename(nullptr, Object);
		}
		else
		{
			Component = NewObject<TComponent>(Object);
			check(Component);
		}

		Component->RegisterComponent();
		return *this;
	}

	template<typename TChildActorType, typename T = TUObject>
	TObjectBuilder<TUObject>& AddChildActorComponentTo()
	{
		static_assert(std::is_base_of_v<AActor, T>, "Can only add components to AActors");
		static_assert(std::is_base_of<AActor, TChildActorType>::value, "Template param must be an Actor type");
		if (!Object)
		{
			UE_LOGF(LogObjectBuilder, Error, "Tried to AddChildActorComponentTo actor after Spawning. ObjectBuilder can only be used to paramaterise pre-spawn");
			return *this;
		}

		auto* Component = NewObject<UChildActorComponent>(Object);
		Component->SetChildActorClass(TChildActorType::StaticClass());
		Component->RegisterComponent();
		return *this;
	}

private:
	TUObject* Object;

	template <typename T>
	bool IsCompatibleWith(const FProperty& Prop) const
	{
		using Raw = std::decay_t<T>;
		if constexpr (std::is_pointer_v<T> && std::is_base_of_v<UObject, std::remove_pointer_t<Raw>>)
		{
			if (auto AsProperty = CastField<FObjectProperty>(&Prop))
			{
				return std::remove_pointer_t<Raw>::StaticClass()->IsChildOf(AsProperty->PropertyClass);
			}
		}
		else if constexpr (TIsTObjectPtr<Raw>::Value)
		{
			if (auto AsProperty = CastField<FObjectProperty>(&Prop))
			{
				return Raw::ElementType::StaticClass()->IsChildOf(AsProperty->PropertyClass);
			}
		}
		else if constexpr (TIsTArray<Raw>::Value)
		{
			if (auto AsProperty = CastField<FArrayProperty>(&Prop))
			{
				return IsCompatibleWith<typename Raw::ElementType>(*AsProperty->Inner);
			}
		}
		else if constexpr (TIsTSet_V<Raw>)
		{
			if (auto AsProperty = CastField<FSetProperty>(&Prop))
			{
				return IsCompatibleWith<typename Raw::ElementType>(*AsProperty->ElementProp);
			}
		}
		else if constexpr (TIsTMap_V<Raw>)
		{
			if (auto AsProperty = CastField<FMapProperty>(&Prop))
			{
				using Key = typename Raw::KeyInitType;
				using Value = typename Raw::ValueInitType;
				return IsCompatibleWith<Key>(*AsProperty->KeyProp) && IsCompatibleWith<Value>(*AsProperty->ValueProp);
			}
		}
		else if constexpr (TIsEnum<Raw>::Value)
		{
			if (auto AsProperty = CastField<FEnumProperty>(&Prop))
			{
				return AsProperty->GetUnderlyingProperty()->GetSize() == sizeof(T);
			}
		}
		else if constexpr (std::is_base_of_v<UObject, Raw>)
		{
			if (auto AsProperty = CastField<FObjectProperty>(&Prop))
			{
				return Raw::StaticClass()->IsChildOf(AsProperty->PropertyClass);
			}
		}
		else if constexpr (std::is_same_v<FVector, Raw>)
		{
			if (auto AsProperty = CastField<FStructProperty>(&Prop))
			{
				return AsProperty->Struct->GetFName() == NAME_Vector;
			}
		}
		else if constexpr (std::is_same_v<FStructProperty, typename TPropType<Raw>::Value>)
		{
			if (auto AsProperty = CastField<FStructProperty>(&Prop))
			{
				return AsProperty->Struct->GetName() == Raw::StaticStruct()->GetName();
			}
		}
		else
		{
			using PropType = typename TPropType<Raw>::Value;

			if (auto AsProperty = CastField<PropType>(&Prop))
			{
				return PropType::StaticClass()->IsChildOf(AsProperty->GetClass());
			}
		}

		return false;
	}

	template<typename T>
	FString GetTypeName() const 
	{
		using Raw = std::decay_t<T>;
		if constexpr (std::is_pointer_v<Raw> && std::is_base_of_v<UObject, std::remove_pointer_t<Raw>>)
		{
			return FString::Printf(TEXT("%s*"), *std::remove_pointer_t<Raw>::StaticClass()->GetName());
		}
		else if constexpr (TIsTObjectPtr<Raw>::Value)
		{
			using PtrType = typename Raw::ElementType;
			return FString::Printf(TEXT("TObjectPtr<%s>"), *GetTypeName<PtrType>());
		}
		else if constexpr (TIsTArray<Raw>::Value)
		{
			using ElementType = typename Raw::ElementType;
			return FString::Printf(TEXT("TArray<%s>"), *GetTypeName<ElementType>());
		}
		else if constexpr (TIsTSet<Raw>::Value)
		{
			using ElementType = typename Raw::ElementType;
			return FString::Printf(TEXT("TSet<%s>"), *GetTypeName<ElementType>());
		}
		else if constexpr (TIsTMap<Raw>::Value)
		{
			using Key = typename Raw::KeyInitType;
			using Value = typename Raw::ValueInitType;
			return FString::Printf(TEXT("TMap<%s, %s>"), *GetTypeName<Key>(), *GetTypeName<Value>());
		}
		else if constexpr (TIsEnum<Raw>::Value)
		{
			return TEXT("EnumProperty");
		}
		else if constexpr (std::is_same_v<FVector, Raw>)
		{
			return TEXT("FVector");
		}
		else if constexpr (std::is_base_of_v<UObject, Raw>)
		{
			return FString::Printf(TEXT("%s"), *Raw::StaticClass()->GetName());
		}
		else if constexpr (std::is_same_v<FStructProperty, typename TPropType<Raw>::Value>)
		{
			return Raw::StaticStruct()->GetName();
		}
		else
		{
			return TNameOf<Raw>::GetName();
		}
	}

	template<typename T> struct TPropType { using Value = FStructProperty; };
	template <> struct TPropType<int8> { using Value = FInt8Property; };
	template <>	struct TPropType<uint8>	{ using Value = FByteProperty; };
	template <>	struct TPropType<int16>	{ using Value = FInt16Property;	};
	template <>	struct TPropType<uint16> { using Value = FUInt16Property; };
	template <>	struct TPropType<int32>	{ using Value = FIntProperty; };
	template <>	struct TPropType<uint32> { using Value = FUInt32Property; };
	template <>	struct TPropType<int64>	{ using Value = FInt64Property;	};
	template <>	struct TPropType<uint64> { using Value = FUInt64Property; };
	template <>	struct TPropType<bool> { using Value = FBoolProperty; };
	template <>	struct TPropType<float>	{ using Value = FFloatProperty;	};
	template <>	struct TPropType<double> { using Value = FDoubleProperty; };
	template <>	struct TPropType<FName>	{ using Value = FNameProperty; };
};

#endif // WITH_AUTOMATION_WORKER
