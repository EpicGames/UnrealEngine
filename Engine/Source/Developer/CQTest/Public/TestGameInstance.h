// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameInstance.h"

#include "TestGameInstance.generated.h"

#define UE_API CQTEST_API

UCLASS(MinimalAPI, config=Game, transient, BlueprintType, Blueprintable)
class UTestGameInstance : public UGameInstance
{
	GENERATED_UCLASS_BODY()

public:
	UE_API void InitializeForTest(UWorld& InWorld);
};

#undef UE_API
