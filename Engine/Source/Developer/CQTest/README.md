# Introduction

Extension of the Unreal Engine FAutomationTestBase to provide test fixtures and common automation testing commands.

# Why CQTest?

There are other valid ways of testing in Unreal engine.  One option is to use the provided macros from Unreal Engine: [docs](https://dev.epicgames.com/documentation/en-us/unreal-engine/write-cplusplus-tests-in-unreal-engine)
```cpp
    IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMinimalTest, "Game.Test", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMinmalTest::RunTest(const FString& Parameters) 
	{
		TestTrue(TEXT("True should be true"), true);
		return true;
	}
```

Unreal Engine also has a [spec test framework](https://dev.epicgames.com/documentation/en-us/unreal-engine/automation-spec-in-unreal-engine), which is inspired by [Behavior Driven Design](https://en.wikipedia.org/wiki/Behavior-driven_development)
```cpp
    DEFINE_SPEC(FMinimalTest, "Game.Test", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	void FMinimalTest::Define() 
	{
		Describe("Assertions", [this]() 
		{
			It("Should pass when testing that true is true", [this]() 
			{
				TestTrue(TEXT("True should be true"), true);
			});
		});
	}
```

With the spec tests, be careful about capturing state
```cpp
    BEGIN_DEFINE_SPEC(FMinimalTest, "Game.Test", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
		uint32 SomeValue = 3;
	END_DEFINE_SPEC

	void FMinimalTest::Define() 
	{
		Describe("Assertions", [this]() 
		{
			uint describeValue = 42;
			It("Has access to members defined on the spec", [this]() 
			{
				TestEqual(TEXT("Class value should be set"), SomeValue, 3);
			});
			
			It("Does not capture variables described inside of lambdas", [this]() 
			{
				TestEqual(TEXT("DescribeValue will now be garbage as it went out of scope"), describeValue, 42);
			});
		});
	}
```

The inspiration for **CQTest** was to add the before/after test abilities, while resetting state between tests automatically.  One of the guiding principles is to make easy things easy.
```cpp
	TEST(MinimalTest, "Game.Test") 
	{
		ASSERT_THAT(IsTrue(true));
	}
	
	TEST_CLASS(MinimalFixture, "Game.Test") 
	{
		uint32 SomeNumber = 0;
		BEFORE_EACH() 
		{
			SomeNumber++;
		}

		TEST_METHOD(MinimalFixture, CanAccessMembers) 
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));  // passes every time
		}
	};
```

# Installation

Inside the project's .Build.cs file, you'll want to add the following to the `PrivateDependencyModuleNames`.  Something like
```csharp
		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"CQTest"  <----
				 }
			);
```

CQTest also has a plugin available that provides a set of tests to validate and document the behavior.  To enable the test plugin navigate to the project's .uproject file and inside you'll need to add the following to the Plugins section
```json
		{
			"Name": "CQTest",
			"Enabled": true
		}
```

# Additional Plugin

CQTest comes with an additonal CQTestEnhancedInput plugin that provides additional input handling functionality to help with testing.  Installation and use of this plugin follows similar steps outlined above for CQTest of adding the module to the list of `PrivateDependencyModuleNames` and adding the plugin to the project's .uproject file.  After installation of the plugin both the input components and input tests will be available to the project.

# Test

To run tests provided by the plugins within the Unreal Editor
- Launch the editor
- Make sure the plugin "Code Quality Tests Unreal Test Plugin" is enabled
- Find the Tools drop down and select Session Frontend
- Navigate to the Automation tab
- By default, the tests should be listed first under "TestFramework.CQTest"
- Select the tests you would like to run and press 'Start Tests'

# Examples

Tests can be as simple as
```cpp
    #include "CQTest.h"
	
	TEST(MinimalTest, "Game.MyGame") 
	{
		ASSERT_THAT(IsTrue(true));
	}
```

For setup and teardown, or common state between multiple tests, or to group related tests, use the `TEST_CLASS` macro.
```cpp
    #include "CQTest.h"
	TEST_CLASS(MyNeatTest, "Game.MyGame") 
	{
		bool SetupCalled = false;
		uint32 SomeNumber = 0;
		Thing* Thing = nullptr;
		
		// Optional static method to be executed before all tests of this TEST_CLASS
		// Should be removed if empty and unused
		BEFORE_ALL() 
		{
			// Perform some logic that is shared with all tests such as loading a level
		}

		BEFORE_EACH() 
		{
			SetupCalled = true;
			SomeNumber++;
			Thing = new Thing();
		}
		
		AFTER_EACH() 
		{
			delete Thing; // Should normally use RAII for things like this
		}

		// Optional static method to be executed after all tests of this TEST_CLASS
		// Should be removed if empty and unused
		AFTER_ALL()
		{
			// Perform cleanup of any resources that was done in the `BEFORE_ALL`.
		}
		
	protected:
		bool UsefulHelperMethod() const 
		{
			return true; 
		}
		
		TEST_METHOD(BeforeRunTest_CallsSetup) 
		{
			ASSERT_THAT(IsTrue(SetupCalled));
		}
		
		TEST_METHOD(ProtectedMembers_AreAccessible) 
		{
			ASSERT_THAT(IsTrue(UsefulHelperMethod()));
		}
		
		TEST_METHOD(DataMembers_BetweenTestRuns_AreReset) 
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));
		}
	};
```

Tests can be tagged using reserved tag macro definitions and they are specified in the format `[TagA][TagB][TagC]`.
Test classes specify common tags that apply to all enclosed methods. In addition, each test method can specify their own tags.
```cpp
    #include "CQTest.h"
	TEST_CLASS_WITH_TAGS(MyTaggedTest, "Game.MyGame.WithTags", "[FeatureA][SpecialTagA]") 
	{
		TEST_METHOD_WITH_TAGS(MyTestMethod, "[MethodSpecificTag]")
		{
			// This method can be filtered by tags FeatureA, SpecialTagA (common as defined by test class) and MethodSpecificTag (method specific tag).
		}
	}
```

In addition to `TEST` and `TEST_CLASS` are 13 additional macros:
 - `TEST_METHOD_WITH_TAGS` - Extends `TEST_METHOD` with an additional argument for test tags.
 - `TEST_WITH_TAGS` - Extends `TEST` with an additional argument for test tags.
 - `TEST_CLASS_WITH_TAGS` - Extends `TEST_CLASS` with an additional argument for test tags.
 - `TEST_CLASS_WITH_ASSERTS` - Macro which allows this test object to use a custom asserter.  More information about how to use this macro can be found [in the section below regarding assertions](#assertions)
 - `TEST_CLASS_WITH_ASSERTS_AND_TAGS` - Extends `TEST_CLASS_WITH_ASSERTS` with an additional argument for test tags.
 - `TEST_CLASS_WITH_BASE` - Macro which allows this test object to inherit from a different test object.  More information about how to use this macro can be found [in the section below regarding custom test classes](#base-test-class)
 - `TEST_CLASS_WITH_BASE_AND_TAGS` - Extends `TEST_CLASS_WITH_BASE` with an additional argument for test tags.
 - `TEST_CLASS_WITH_FLAGS` - Macro which allows the use of different automation test flags to be specified.  Useful for when tests can only run under a certain context or grouped under a specific filter.  The default flags for `TEST` and `TEST_CLASS` are `EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter`.  Additional information regarding the available flags can be found [in the online documentation](https://dev.epicgames.com/documentation/en-us/unreal-engine/API/Runtime/Core/Misc/EAutomationTestFlags/Type)
 - `TEST_CLASS_WITH_FLAGS_AND_TAGS` - Extends - `TEST_CLASS_WITH_FLAGS` with an additional argument for test tags.
 - `TEST_CLASS_WITH_BASE_AND_FLAGS` - Macro which allows for this test object to inherit from a different test object and allows for custom automation test flags to be specified.
 - `TEST_CLASS_WITH_BASE_AND_FLAGS_AND_TAGS` - Extends - `TEST_CLASS_WITH_BASE_AND_FLAGS` with an additional argument for test tags.
 - `_TEST_CLASS_IMPL_EXT` - Base macro which is used by the above macros to specify a custom asserter, a test object to inherit from a different test object, and allows for custom automation test flags to be specified as well as test tags.
 - `_TEST_CLASS_IMPL` - Expands `_TEST_CLASS_IMPL_EXT` with default tags.

Test Directory determines where in the Automation tab the tests appear.  In the example above, we specify "Game.MyGame", but you may also have an auto-generated test directory based on the folder structure.
```cpp
	TEST_CLASS(MyNeatTest, GenerateTestDirectory)
	{
	};

	TEST_CLASS(MyNeatTest, "Game.Test.[GenerateTestDirectory].Validation")
	{
	};
```
In the above examples, if `MyNeatTest` is located within a plugin of a project with the path **MyProject/Plugins/GameTests/Source/GameTests/Private/NeatTest.cpp** then the generated test names will be `MyProject.Plugins.GameTests.MyNeatTest` and `Game.Test.MyProject.Plugins.GameTests.MyNeatTest.Validation` respectively.

Constructors (and destructors) are available.  Destructors shouldn't throw, and you shouldn't put assertions in them (as they are called after the testing framework is done with the test).
```cpp
	TEST_CLASS(SomeTestClass, "Game.Test")
	{
		bool bConstructed = false;
		SomeTestClass()
			: bConstructed(true)
		{
		}
		
		TEST_METHOD(ConstructorIsCalled)
		{
			ASSERT_THAT(IsTrue(bConstructed));
		}
	};
```	

Latent actions are supported with the `TEST_CLASS` macro.  Each step will complete all latent actions before moving to the next.  If an assertion is raised during a latent action, then no further latent actions will be processed.  The `AFTER_EACH` method will still be invoked though.
```cpp
    TEST_CLASS(LatentActionTest, "Game.Test") 
	{
		uint32 calls = 0;
		BEFORE_EACH() 
		{
			AddCommand(new FExecute([&]() { calls++; }));
		}
		
		AFTER_EACH() 
		{
			AddCommand(new FExecute([&]() { calls++; })); // executed after the next line, as it is a latent action
			ASSERT_THAT(AreEqual(2, calls));
		}
		
		TEST_METHOD(PerformLatentAction) 
		{
			ASSERT_THAT(AreEqual(1, calls));
			AddCommand(new FExecute([&]() { calls++; }));
		}
	};
```

**CQTest** provides the following additional latent actions:
 - `FExecute` - Action that executes only once.
 - `FWaitUntil` - Action that executes over multiple ticks until either completion or the duration exceeds the timeout.  Action will fail if the condition cannot be satisfied before timing out.
 - `FWaitDelay` - Action that waits a specified duration.
   - **CAUTION:** Using a timed-wait can introduce flakiness due to variable runtimes and the above `FWaitUntil` should be used instead.
 - `FRunSequence` - Action which ensures that a collection of latent actions occur in order, and only after all previous actions have finished.
 - `TAsyncExecute` - Action that executes some asynchronous code, which produces a `TAsyncResult`, and waits for its completion without blocking the Game thread. The action will fail if the result of the asynchronous call is not ready within the specified timeout. The result can be processed in the nested `FExecute` or `FWaitUntil` action.

Also available for commands is a fluent command builder
```cpp
	TEST_METHOD(SomeTest) 
	{
		TestCommandBuilder
			.Do([&]() { StepOne(); })
			.Then([&]() { StepTwo(); })
			.Until([&]() { return StepThreeComplete(); })
			.Then([&]() { ASSERT_THAT(IsTrue(SomethingImportant)); });
	}
	
	TEST_METHOD(AsyncFuncTest) 
	{
		/**
		 * Assuming the following declarations of async functions:
		 *
		 *	TAsyncResult<void> AsyncStepOne();
		 *	TAsyncResult<void> AsyncStepTwo();
		 *	TAsyncResult<bool> AsyncStepThree();
		 *	TAsyncResult<SomeResultType> AsyncStepFour();
		 */

		TestCommandBuilder
			.DoAsync<void>([&]() { return AsyncStepOne(); })
			.ThenAsync<void>([&]() { return AsyncStepTwo(); })
			.ThenAsync<bool>(
				// This lambda is called to start an async action
				[&]() { return AsyncStepThree(); },
				// This lambda is called to process the result, once it's ready
				[&](bool bResult) { ASSERT_THAT(IsTrue(bResult)); }
			)
			.UntilAsync<SomeResultType>(
				// This lambda is called to start an async action
				[&]() { return AsyncStepFour(); },
				// Once the result is ready, this lambda is called multiple times until the condition is satisfied or the duration exceeds the timeout
				[&](const SomeResultType& Result) { return Result.StepFourComplete(); }
			)
			.Then([&]() { ASSERT_THAT(IsTrue(SomethingImportant)); });
	}
```

The command builder provides commands which wrap around the above mentioned latent actions.  The following commands are made available:
 - `Do`/`Then` - Commands which adds the `FExecute` latent action with the provided lambda to be executed.
 - `StartWhen`/`Until` - Commands which adds the `FWaitUntil` latent action with the provided lambda to be evaluated.
 - `DoAsync`/`ThenAsync` - Commands which add the `TAsyncExecute` latent action with two provided lambdas: the first one to execute an async function, and the second one (optional) to process the result of the async function in the `FExecute` latent action.
 - `UntilAsync` - Command which adds the `TAsyncExecute` latent action with two provided lambdas: the first one to execute an async function, and the second one to process the result of the async function in the `FWaitUntil` latent action.
 - `WaitDelay` - Command which waits a specified duration before continuing.  
   - **CAUTION:** Using a timed-wait can introduce flakiness due to variable runtimes and the above `StartWhen`/`Until` commands should be used instead.
 - `OnTearDown`/`CleanUpWith` - Commands which adds the `FExecute` latent action with the provided lambda to be executed after the test.  Can be called multiple times to add multiple clean up latent actions.
   - **NOTE:** Latent actions added using the `OnTearDown`/`CleanUpWith` will be run in reverse order (i.e. Last in, first out)

The framework will ensure that all of those commands happen in order using a future pattern.
Similarly, the framework will ensure that a test can await a ticking object.  See `GameObjectsTickTest` for an example

**CAUTION:** The framework does not currently support adding latent actions from within latent actions.
Instead, it is better to add the actions as a series of self-contained steps.

# Extending the framework

The framework has been designed to allow for extensions in a couple areas.  In-code examples can be found within the CQTest plugin used to test the framework. See _/Engine/Plugins/Tests/CQTest/Source/CQTestTests/Private/ExtensionTests.cpp_ for in-code examples.

## Test Settings

The framework also comes with settings that can be applied per project.  The settings are saved to the project's Engine configuration file located in _Config/DefaultEngine.ini_ under the section `[/Script/CQTest.CQTestSettings]`. Some of the available settings are:
- `TestFramework.CQTest.CommandTimeout` - Timeout value, in seconds, that is applied to commands that wait on an action to be evaluated to true and no user defined timeout is specified.
- `TestFramework.CQTest.CommandTimeout.Network` - Timeout value, in seconds, that is applied to commands that wait on a PIENetworkComponent action to be evaluated to true and no user defined timeout is specified.
- `TestFramework.CQTest.CommandTimeout.MapTest` - Timeout value, in seconds, that is applied to the `FMapTestSpawner::AddWaitUntilLoadedCommand` method for loading a map and starting a PIE session.

The above mentioned settings are stored as Console Variables and can also be adjusted within the Editor or Game build targets through the console.
**NOTE:** Adjusting the variables through the console will only have the value applied through the duration of the application run and will reset to the defaults if no prior saved settings exist upon exit.  Changes must be modified through the Editor by going into the **Project Settings** and then finding **CQ Test Settings** under the **Engine** category.

## Test Components

This testing framework embraces composition over inheritence.  Creating new components should be the default mechanism for extending the framework.  Some of the components available to you are:
 - `SpawnHelper` - Eases the ability to spawn actors and other objects.  Implemented by `ActorTestSpawner` and `MapTestSpawner`.
 - `ActorTestSpawner` - Creates a minimal `UWorld` for a test to spawn actors, and manages their despawning.
 - `MapTestSpawner` - Can create a temporary map or open a specified level.  Allows tests to spawn actors in that world.
 - `CQTestBlueprintHelper` - Eases the ability for a test to spawn Blueprint objects, intended to be used with `MapTestSpawner`.
   - **NOTE:** Loading Blueprint assets is only intended to work within the Editor context.  Tests that make use of the `CQTestBlueprintHelper` should specify the `EAutomationTestFlags::EditorContext` flag.
 - `PIENetworkComponent` - Allows tests to create a server and a collection of clients.  Good for testing replication.
    - **NOTE:** The `PIENetworkComponent` sets up a Server and Client PIE instance which is only usable within the Editor context.  Tests that make use of the `PIENetworkComponent` should specify the `EAutomationTestFlags::EditorContext` flag.
 - `InputTestActions` - Allows tests to inject `InputActions` to the `Pawn`.
 - `CQTestSlateComponent` - Allows tests to get notified when the UI has been updated.

### Notable Changes

#### 5.6
- Default timeout durations are able to be configured per project via the **Project Settings** within the Editor or through CVars.
   
#### 5.5
- **CQTest** - The core **CQTest** framework has been extracted out from the plugin and is now an Engine Module. While the plugin still exists and is used to test that the core functionality works; it is deprecated and not necessary to be included as part of the project. No action needs to be taken for existing projects to have the core **CQTest** framework available.
- **CQTestEnhancedInput** - **BREAKING CHANGE** Due to the move of **CQTest** being a module, the components that used the **EnhancedInput** plugin had to be extracted out into a separate plugin. **EnhancedInput** is currently an Engine plugin, similar to what **CQTest** used to be. Because both were considered Engine plugins, they were able to reference each other. With the move of **CQTest** being in the Engine and cannot reference an Engine plugin, there is a need to add `CQTestEnhancedInput` to the project's Build.cs file, similar to how `CQTest` was added. This will only impact the project if the `InputTestAction` component was being used, but does not have an impact on the core **CQTest** framework.
- `CQTestBlueprintHelper` - **DEPRECATED IN 5.5** Eases the ability for a test to spawn Blueprint objects, intended to be used with `MapTestSpawner`.
   - **NOTE:** Loading Blueprint assets is only intended to work within the Editor context.  Tests that make use of the `CQTestBlueprintHelper` should specify the `EAutomationTestFlags::EditorContext` flag.

## Test Helpers

This testing framework provides the following helper objects and methods:
 - `FAssetFilterBuilder` - Helps create an asset filter to be used with either the `CQTestAssetHelper` namespace methods or when searching through the `AssetRegistry` directly.
 - `CQTestAssetHelper` - Namespace with helper methods used to search for either asset package paths or Blueprints by name or by building a filter from the `FAssetFilterBuilder`.

## Assertions

Not all platforms support exceptions, and so the assertions are unable to rely on them.
There are a few options here:
 - We could just throw exceptions, and only run tests on platforms which support exceptions
 - We could return a `[[nodiscard]]` bool to encourage checking each assertion and returning if it fails
 - We could return a normal bool and rely on people to check it when it's important.

Exceptions have the advantage of working in helper functions and lambdas, as well as not depending on human diligence.
A normal bool is less noisy, and allows developers to use intellisense, but is more error prone
The default implementation used is the `[[nodiscard]]` bool, with a helper macro `ASSERT_THAT` which does the early return check for you.

You can use your own types within the `Assert.AreEqual` and `Assert.AreNotEqual` methods assuming you have the `==` and `!=` operators defined as needed.
In addition, the error message will print out the string version of your type, assuming you have a `ToString` method defined as well.  The framework will complain if it doesn't know how to print your value.
Below is a simple example.
```cpp
struct MyCustomType
{
	int32 Value;
	bool operator==(const MyCustomType& other) const
	{
		return Value == other.Value;
	}
	bool operator!=(const MyCustomType& other) const
	{
		return !(*this == other);
	}
	
	FString ToString() const
	{
		//your to string logic
		return FString();
	}
};
enum struct MyCustomEnum
{
	Red, Green, Blue
};
template<>
FString CQTestConvert::ToString(const MyCustomEnum&)
{
	//your to string logic
	return FString();
}
```

You are able to customize the assertions which are available, and how they behave.
Below is some untested example code to inspire ideas

```cpp
	struct FluentAsserter
	{
	private:
		int CurrentIntValue = 0;
		TArray<FString> Errors;
		FAutomationTestBase& TestRunner;
		
	public:
		FluentAsserter(FAutomationTestBase& InTestRunner)
			: TestRunner(InTestRunner)
		{
		}
		
		~FluentAsserter()
		{
			for(const auto& error : Errors)
			{
				TestRunner.AddError(error);
			}
		}
		
		FluentAsserter& That(int value)
		{
			CurrentIntValue = value;
			return *this;
		}
		
		FluentAsserter& Equals(int value)
		{
			if(CurrentIntValue != value)
			{
				Errors.Add(FString::Printf("%d != %d", CurrentIntValue, value));
			}
			return *this;
		}
	};
```

From here, you could create macros your studio uses to create tests
```cpp
	#define MY_STUDIO_TEST_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_ASSERTS(_ClassName, _TestDir, FluentAsserter)
	#define MY_STUDIO_TEST(_TestName, _TestDir) \
	MY_STUDIO_TEST_CLASS(_TestName, _TestDir) \
	{ \
		TEST_METHOD(_TestName##_Method); \
	};\
	void _TestName::_TestName##_Method()
```

## Base test class

Similarly there may be a use case to create many tests which have the same member variables or helper methods.  This can be implemented by extending the test class
```cpp
	template<typename Derived, typename AsserterType>
	struct ActorTest : public Test<Derived, AsserterType>
	{
		SpawnHelper Spawner;
	};
```

And creating a macro which uses it
```cpp
	#define ACTOR_TEST(_ClassName, _TestDir) TEST_CLASS_WITH_BASE(_ClassName, _TestDir, ActorTest)
```

# Contribute

Improvements like bug fixes and extensions are welcome when accompanied by unit tests.
